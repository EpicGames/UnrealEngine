{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SlateInspectorToolset",
	"Description": "Slate UI automation and inspection tools.",
	"NoRedist": true,
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SlateInspectorToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		},
		{
			"Name": "SlateInspectorToolsetTests",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
