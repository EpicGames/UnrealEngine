// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolset.h"

#include "SlateInspectorToolsetObserverManager.h"
#include "SlateInspectorToolsetRefCache.h"
#include "SlateInspectorToolsetSnapshotRenderer.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Logging/StructuredLog.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

DEFINE_LOG_CATEGORY_STATIC(LogSlateInspectorToolset, Log, All);

namespace SlateInspectorToolsetPrivate
{

/** Escape a string for safe embedding in a JSON string value (RFC 8259). */
FString EscapeJsonString(const FString& Input)
{
	FString Result;
	Result.Reserve(Input.Len() + 16);
	for (TCHAR Character : Input)
	{
		switch (Character)
		{
		case TEXT('"'):  Result += TEXT("\\\""); break;
		case TEXT('\\'): Result += TEXT("\\\\"); break;
		case TEXT('\b'): Result += TEXT("\\b");  break;
		case TEXT('\f'): Result += TEXT("\\f");  break;
		case TEXT('\n'): Result += TEXT("\\n");  break;
		case TEXT('\r'): Result += TEXT("\\r");  break;
		case TEXT('\t'): Result += TEXT("\\t");  break;
		default:
			// Escape remaining ASCII control characters (U+0000..U+001F) per RFC 8259.
			if (Character < 0x20)
			{
				Result += FString::Printf(TEXT("\\u%04x"), static_cast<uint32>(Character));
			}
			else
			{
				Result += Character;
			}
			break;
		}
	}
	return Result;
}

} // namespace SlateInspectorToolsetPrivate

// ── Helpers ──────────────────────────────────────────────────────────

TSharedPtr<SWidget> USlateInspectorToolset::ResolveRefOrWarn(const FString& Ref)
{
	TSharedPtr<SWidget> Widget = FSlateInspectorToolsetRefCache::Get().ResolveRef(Ref);
	if (!Widget.IsValid())
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Could not resolve ref '{Ref}'", Ref);
	}
	return Widget;
}

FVector2D USlateInspectorToolset::GetWidgetScreenCenter(TSharedRef<SWidget> Widget)
{
	const FGeometry& Geometry = Widget->GetTickSpaceGeometry();
	return FVector2D(Geometry.GetAbsolutePositionAtCoordinates(FVector2D(0.5, 0.5)));
}

FKey USlateInspectorToolset::ParseMouseButtonKey(const FString& Button)
{
	if (Button.Equals(TEXT("right"), ESearchCase::IgnoreCase))
	{
		return EKeys::RightMouseButton;
	}
	if (Button.Equals(TEXT("middle"), ESearchCase::IgnoreCase))
	{
		return EKeys::MiddleMouseButton;
	}
	return EKeys::LeftMouseButton;
}

bool USlateInspectorToolset::SimulateClick(TSharedRef<SWidget> Widget, FKey MouseButton, bool bDoubleClick, const FModifierKeysState& ModifierKeys)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	// Find the window containing this widget by walking up its parent chain
	// to the nearest SWindow. FindWidgetWindow can return the wrong window
	// for some widget types, and LocateWindowUnderMouse depends on z-order.
	TSharedPtr<SWindow> Window;
	TSharedPtr<SWidget> Current = Widget;
	static const FName WindowType(TEXT("SWindow"));
	while (Current.IsValid())
	{
		if (Current->GetType() == WindowType)
		{
			Window = StaticCastSharedPtr<SWindow>(Current);
			break;
		}
		Current = Current->GetParentWidget();
	}
	if (!Window.IsValid())
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Could not find window containing widget");
		return false;
	}

	FVector2D ScreenCenter = GetWidgetScreenCenter(Widget);
	FSlateApplication::Get().SetCursorPos(ScreenCenter);

	// Send a mouse move so Slate updates its hover tracking, widget path
	// cache, and window z-order for hit-testing.
	FPointerEvent MoveEvent(
		0,
		ScreenCenter,
		ScreenCenter,
		TSet<FKey>(),
		EKeys::Invalid,
		0.0f,
		ModifierKeys);
	FSlateApplication::Get().ProcessMouseMoveEvent(MoveEvent);

	TSet<FKey> PressedButtons;
	PressedButtons.Add(MouseButton);

	FPointerEvent MouseEvent(
		0,
		ScreenCenter,
		ScreenCenter,
		PressedButtons,
		MouseButton,
		0.0f,
		ModifierKeys);

	if (bDoubleClick)
	{
		FSlateApplication::Get().ProcessMouseButtonDoubleClickEvent(Window->GetNativeWindow(), MouseEvent);
	}
	else
	{
		FSlateApplication::Get().ProcessMouseButtonDownEvent(Window->GetNativeWindow(), MouseEvent);
	}

	// Release for both paths. ProcessMouseButtonDoubleClickEvent implies a
	// mouse-down, so we must release to avoid leaving widgets in pressed state.
	FPointerEvent MouseUpEvent(
		0,
		ScreenCenter,
		ScreenCenter,
		TSet<FKey>(),
		MouseButton,
		0.0f,
		ModifierKeys);
	FSlateApplication::Get().ProcessMouseButtonUpEvent(MouseUpEvent);

	return true;
}

// ── Snapshot tools ───────────────────────────────────────────────────

FString USlateInspectorToolset::Snapshot(const FString& Ref, int32 MaxDepth, bool bIncludeSourceLocations)
{
	MaxDepth = FMath::Max(MaxDepth, 0);

	TSharedPtr<SWidget> Root;
	if (!Ref.IsEmpty())
	{
		Root = ResolveRefOrWarn(Ref);
		if (!Root.IsValid())
		{
			return TEXT("Error: Could not resolve ref");
		}
	}

	// If an observer covers this root, return its cached snapshot.
	if (!bIncludeSourceLocations)
	{
		FString CachedSnapshot = FSlateInspectorToolsetObserverManager::Get().FindMatchingObserverSnapshot(Root, MaxDepth);
		if (!CachedSnapshot.IsEmpty())
		{
			return CachedSnapshot;
		}
	}

	// No observer covers this request. Render fresh without resetting the cache
	// (observers may be maintaining refs for other subtrees).
	return FSlateInspectorToolsetSnapshotRenderer::Render(Root, MaxDepth, bIncludeSourceLocations, /* bResetCache */ false);
}

FString USlateInspectorToolset::Observe(const FString& Ref, int32 MaxDepth)
{
	MaxDepth = FMath::Max(MaxDepth, 0);

	TSharedPtr<SWidget> RootWidget;
	if (!Ref.IsEmpty())
	{
		RootWidget = ResolveRefOrWarn(Ref);
		if (!RootWidget.IsValid())
		{
			return TEXT("Error: Could not resolve ref");
		}
	}

	return FSlateInspectorToolsetObserverManager::Get().AddObserver(RootWidget, MaxDepth);
}

bool USlateInspectorToolset::Unobserve(const FString& Identifier)
{
	return FSlateInspectorToolsetObserverManager::Get().RemoveObserver(Identifier);
}

FString USlateInspectorToolset::ListObservers()
{
	TArray<FSlateInspectorToolsetObserver> Observers = FSlateInspectorToolsetObserverManager::Get().GetObservers();

	FString Result = TEXT("[");
	for (int32 Index = 0; Index < Observers.Num(); ++Index)
	{
		const FSlateInspectorToolsetObserver& Observer = Observers[Index];
		if (Index > 0)
		{
			Result += TEXT(", ");
		}

		FString RootRef;
		if (!Observer.bRoot)
		{
			TSharedPtr<SWidget> RootWidget = Observer.RootWidget.Pin();
			if (RootWidget.IsValid())
			{
				RootRef = FSlateInspectorToolsetRefCache::Get().FindRef(RootWidget.ToSharedRef());
			}
		}

		Result += FString::Printf(
			TEXT("{\"identifier\": \"%s\", \"root\": %s, \"rootRef\": \"%s\", \"maxDepth\": %d, \"cachedSnapshotSize\": %d}"),
			*SlateInspectorToolsetPrivate::EscapeJsonString(Observer.Identifier),
			Observer.bRoot ? TEXT("true") : TEXT("false"),
			*SlateInspectorToolsetPrivate::EscapeJsonString(RootRef),
			Observer.MaxDepth,
			Observer.CachedSnapshotText.Len());
	}
	Result += TEXT("]");

	return Result;
}

FToolsetImage USlateInspectorToolset::Screenshot(const FString& Ref)
{
	FToolsetImage Result;

	if (!FSlateApplication::IsInitialized())
	{
		return Result;
	}

	TSharedPtr<SWidget> Widget;
	if (!Ref.IsEmpty())
	{
		Widget = ResolveRefOrWarn(Ref);
	}
	else
	{
		// Use the active top-level window.
		TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
		Widget = ActiveWindow;
	}

	if (!Widget.IsValid())
	{
		return Result;
	}

	TArray<FColor> ColorData;
	FIntVector Size;
	if (FSlateApplication::Get().TakeScreenshot(Widget.ToSharedRef(), ColorData, Size))
	{
		Result.SetFromBitmap(ColorData, FIntPoint(Size.X, Size.Y));
	}

	return Result;
}

// ── Action tools ─────────────────────────────────────────────────────

bool USlateInspectorToolset::Click(const FString& Ref, const FString& Button, bool DoubleClick, const FSlateInspectorToolsetModifierKeys& Modifiers)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	TSharedPtr<SWidget> Widget = ResolveRefOrWarn(Ref);
	if (!Widget.IsValid())
	{
		return false;
	}

	return SimulateClick(Widget.ToSharedRef(), ParseMouseButtonKey(Button), DoubleClick, Modifiers.ToModifierKeysState());
}

bool USlateInspectorToolset::Hover(const FString& Ref)
{
	TSharedPtr<SWidget> Widget = ResolveRefOrWarn(Ref);
	if (!Widget.IsValid() || !FSlateApplication::IsInitialized())
	{
		return false;
	}

	FVector2D ScreenCenter = GetWidgetScreenCenter(Widget.ToSharedRef());
	FSlateApplication::Get().SetCursorPos(ScreenCenter);

	// SetCursorPos alone does not update Slate's hover tracking.
	// Send a mouse move event so the widget receives OnMouseEnter and IsHovered() becomes true.
	FModifierKeysState ModifierKeys;
	FPointerEvent MouseMoveEvent(
		0, ScreenCenter, ScreenCenter,
		TSet<FKey>(), FKey(), 0.0f, ModifierKeys);
	FSlateApplication::Get().ProcessMouseMoveEvent(MouseMoveEvent);

	return true;
}

bool USlateInspectorToolset::Type(const FString& Ref, const FString& Text, bool Submit)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	TSharedPtr<SWidget> Widget = ResolveRefOrWarn(Ref);
	if (!Widget.IsValid())
	{
		return false;
	}

	// Focus the target widget.
	FSlateApplication::Get().SetKeyboardFocus(Widget, EFocusCause::SetDirectly);

	if (!Widget->HasKeyboardFocus())
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Could not focus widget for typing");
		return false;
	}

	FModifierKeysState ModifierKeys;

	// Send each character as a key char event.
	for (TCHAR Character : Text)
	{
		FCharacterEvent CharEvent(Character, ModifierKeys, 0, false);
		FSlateApplication::Get().ProcessKeyCharEvent(CharEvent);
	}

	if (Submit)
	{
		FKeyEvent KeyDownEvent(EKeys::Enter, ModifierKeys, 0, false, 0, 0);
		FSlateApplication::Get().ProcessKeyDownEvent(KeyDownEvent);
		FKeyEvent KeyUpEvent(EKeys::Enter, ModifierKeys, 0, false, 0, 0);
		FSlateApplication::Get().ProcessKeyUpEvent(KeyUpEvent);
	}

	return true;
}

bool USlateInspectorToolset::PressKey(const FString& Key)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	// Parse modifier prefixes: "Ctrl+Shift+Alt+Meta+KeyName"
	bool bShift = false;
	bool bControl = false;
	bool bAlt = false;
	bool bCommand = false;
	FString Remaining = Key;

	while (true)
	{
		if (Remaining.StartsWith(TEXT("Shift+"), ESearchCase::IgnoreCase))
		{
			bShift = true;
			Remaining.RightChopInline(6);
		}
		else if (Remaining.StartsWith(TEXT("Ctrl+"), ESearchCase::IgnoreCase) || Remaining.StartsWith(TEXT("Control+"), ESearchCase::IgnoreCase))
		{
			bControl = true;
			Remaining.RightChopInline(Remaining.StartsWith(TEXT("Ctrl+"), ESearchCase::IgnoreCase) ? 5 : 8);
		}
		else if (Remaining.StartsWith(TEXT("Alt+"), ESearchCase::IgnoreCase))
		{
			bAlt = true;
			Remaining.RightChopInline(4);
		}
		else if (Remaining.StartsWith(TEXT("Meta+"), ESearchCase::IgnoreCase) || Remaining.StartsWith(TEXT("Cmd+"), ESearchCase::IgnoreCase))
		{
			bCommand = true;
			Remaining.RightChopInline(Remaining.StartsWith(TEXT("Meta+"), ESearchCase::IgnoreCase) ? 5 : 4);
		}
		else
		{
			break;
		}
	}

	FKey ParsedKey(*Remaining);

	if (!ParsedKey.IsValid())
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Unknown key '{Key}'", Key);
		return false;
	}

	FModifierKeysState ModifierKeys(bShift, bShift, bControl, bControl, bAlt, bAlt, bCommand, bCommand, false);
	FKeyEvent KeyDownEvent(ParsedKey, ModifierKeys, 0, false, 0, 0);
	FSlateApplication::Get().ProcessKeyDownEvent(KeyDownEvent);

	// Some keys are handled via OnKeyChar rather than OnKeyDown (e.g.
	// BackSpace in SEditableText). Normally the OS translates WM_KEYDOWN
	// into WM_CHAR, but since we bypass the OS message loop we must send
	// the character event ourselves. Only send for unmodified keys (Ctrl/Alt
	// combos are handled purely via OnKeyDown).
	if (!bControl && !bAlt && !bCommand)
	{
		static const TMap<FKey, TCHAR> KeyToCharacter = {
			{ EKeys::BackSpace, TCHAR('\b') },
			{ EKeys::Tab,       TCHAR('\t') },
			{ EKeys::Enter,     TCHAR('\r') },
			{ EKeys::Escape,    TCHAR('\x1b') },
		};

		if (const TCHAR* CharCode = KeyToCharacter.Find(ParsedKey))
		{
			FCharacterEvent CharEvent(*CharCode, ModifierKeys, 0, false);
			FSlateApplication::Get().ProcessKeyCharEvent(CharEvent);
		}
	}

	FKeyEvent KeyUpEvent(ParsedKey, ModifierKeys, 0, false, 0, 0);
	FSlateApplication::Get().ProcessKeyUpEvent(KeyUpEvent);

	return true;
}

bool USlateInspectorToolset::SelectOption(const FString& Ref, const FString& Value)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	TSharedPtr<SWidget> Widget = ResolveRefOrWarn(Ref);
	if (!Widget.IsValid())
	{
		return false;
	}

	// Capture the set of windows before opening, so we can identify the new dropdown popup.
	TArray<TSharedRef<SWindow>> WindowsBefore;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(WindowsBefore);

	// Click to open the combobox dropdown.
	if (!SimulateClick(Widget.ToSharedRef(), EKeys::LeftMouseButton, false))
	{
		return false;
	}

	// Pump Slate twice: first tick creates the popup window, second tick
	// lays out and materializes the list view items inside it.
	// Guard against reentrancy if we're already inside a Slate tick.
	if (!ensure(!FSlateApplication::Get().IsProcessingInput()))
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: SelectOption called during input processing, cannot tick");
		return false;
	}
	FSlateApplication::Get().Tick();
	FSlateApplication::Get().Tick();

	static const FName TextBlockType(TEXT("STextBlock"));
	TSharedPtr<SWidget> MatchingTextBlock;
	TArray<TSharedRef<SWindow>> VisibleWindows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);

	// Search newly appeared windows first (the dropdown popup), then fall back
	// to all windows. This avoids matching text in the combobox's own label.
	TSet<TSharedRef<SWindow>> WindowsBeforeSet(WindowsBefore);
	VisibleWindows.Sort([&WindowsBeforeSet](const TSharedRef<SWindow>& A, const TSharedRef<SWindow>& B)
	{
		bool bAIsNew = !WindowsBeforeSet.Contains(A);
		bool bBIsNew = !WindowsBeforeSet.Contains(B);
		return bAIsNew > bBIsNew;
	});

	for (const TSharedRef<SWindow>& Window : VisibleWindows)
	{
		TArray<TSharedRef<SWidget>> Queue;
		FChildren* WindowChildren = Window->GetChildren();
		for (int32 ChildIndex = 0; ChildIndex < WindowChildren->Num(); ++ChildIndex)
		{
			Queue.Add(WindowChildren->GetChildAt(ChildIndex));
		}

		int32 QueueIndex = 0;
		while (QueueIndex < Queue.Num())
		{
			TSharedRef<SWidget> Current = Queue[QueueIndex++];

			if (Current->GetType() == TextBlockType)
			{
				FText WidgetText = StaticCastSharedRef<STextBlock>(Current)->GetText();
				if (WidgetText.ToString() == Value)
				{
					MatchingTextBlock = Current;
					break;
				}
			}

			FChildren* CurrentChildren = Current->GetChildren();
			for (int32 ChildIndex = 0; ChildIndex < CurrentChildren->Num(); ++ChildIndex)
			{
				Queue.Add(CurrentChildren->GetChildAt(ChildIndex));
			}
		}

		if (MatchingTextBlock.IsValid())
		{
			break;
		}
	}

	if (!MatchingTextBlock.IsValid())
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Could not find option '{Value}'", Value);
		return false;
	}

	return SimulateClick(MatchingTextBlock.ToSharedRef(), EKeys::LeftMouseButton, false);
}

bool USlateInspectorToolset::Drag(const FString& StartRef, const FString& EndRef, const FSlateInspectorToolsetModifierKeys& Modifiers)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	TSharedPtr<SWidget> StartWidget = ResolveRefOrWarn(StartRef);
	TSharedPtr<SWidget> EndWidget = ResolveRefOrWarn(EndRef);
	if (!StartWidget.IsValid() || !EndWidget.IsValid())
	{
		return false;
	}

	// Walk parent chain to find the containing SWindow (same approach as SimulateClick).
	TSharedPtr<SWindow> StartWindow;
	TSharedPtr<SWidget> WalkCurrent = StartWidget;
	static const FName WindowType(TEXT("SWindow"));
	while (WalkCurrent.IsValid())
	{
		if (WalkCurrent->GetType() == WindowType)
		{
			StartWindow = StaticCastSharedPtr<SWindow>(WalkCurrent);
			break;
		}
		WalkCurrent = WalkCurrent->GetParentWidget();
	}
	if (!StartWindow.IsValid())
	{
		UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Could not find window for drag source");
		return false;
	}

	FVector2D StartPosition = GetWidgetScreenCenter(StartWidget.ToSharedRef());
	FVector2D EndPosition = GetWidgetScreenCenter(EndWidget.ToSharedRef());

	const FModifierKeysState ModifierKeys = Modifiers.ToModifierKeysState();
	TSet<FKey> PressedButtons;
	PressedButtons.Add(EKeys::LeftMouseButton);

	// Move cursor to start, update hover tracking, then press.
	FSlateApplication::Get().SetCursorPos(StartPosition);
	FPointerEvent StartMoveEvent(
		0, StartPosition, StartPosition,
		TSet<FKey>(), EKeys::Invalid, 0.0f, ModifierKeys);
	FSlateApplication::Get().ProcessMouseMoveEvent(StartMoveEvent);

	FPointerEvent MouseDownEvent(
		0, StartPosition, StartPosition,
		PressedButtons, EKeys::LeftMouseButton, 0.0f, ModifierKeys);
	FSlateApplication::Get().ProcessMouseButtonDownEvent(StartWindow->GetNativeWindow(), MouseDownEvent);

	// Move cursor to end.
	FPointerEvent MouseMoveEvent(
		0, EndPosition, StartPosition,
		PressedButtons, EKeys::Invalid, 0.0f, ModifierKeys);
	FSlateApplication::Get().ProcessMouseMoveEvent(MouseMoveEvent);
	FSlateApplication::Get().SetCursorPos(EndPosition);

	// Release at end position.
	FPointerEvent MouseUpEvent(
		0, EndPosition, EndPosition,
		TSet<FKey>(), EKeys::LeftMouseButton, 0.0f, ModifierKeys);
	FSlateApplication::Get().ProcessMouseButtonUpEvent(MouseUpEvent);

	return true;
}

// ── Window tools ─────────────────────────────────────────────────────

FString USlateInspectorToolset::Windows(const FString& Action, int32 Index)
{
	if (!FSlateApplication::IsInitialized())
	{
		return TEXT("Error: Slate not initialized");
	}

	TArray<TSharedRef<SWindow>> VisibleWindows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);

	if (Action.Equals(TEXT("list"), ESearchCase::IgnoreCase))
	{
		FString Result = TEXT("[");
		for (int32 WindowIndex = 0; WindowIndex < VisibleWindows.Num(); ++WindowIndex)
		{
			const TSharedRef<SWindow>& Window = VisibleWindows[WindowIndex];
			if (WindowIndex > 0)
			{
				Result += TEXT(", ");
			}
			FString Title = SlateInspectorToolsetPrivate::EscapeJsonString(Window->GetTitle().ToString());
			Result += FString::Printf(TEXT("{\"index\": %d, \"title\": \"%s\"}"),
				WindowIndex,
				*Title);
		}
		Result += TEXT("]");
		return Result;
	}

	if (Action.Equals(TEXT("select"), ESearchCase::IgnoreCase))
	{
		if (Index >= 0 && Index < VisibleWindows.Num())
		{
			VisibleWindows[Index]->BringToFront();
			FSlateApplication::Get().SetAllUserFocus(VisibleWindows[Index], EFocusCause::SetDirectly);
			return TEXT("OK");
		}
		return FString::Printf(TEXT("Error: Window index %d out of range (0..%d)"), Index, VisibleWindows.Num() - 1);
	}

	if (Action.Equals(TEXT("close"), ESearchCase::IgnoreCase))
	{
		if (Index >= 0 && Index < VisibleWindows.Num())
		{
			VisibleWindows[Index]->RequestDestroyWindow();
			return TEXT("OK");
		}
		return FString::Printf(TEXT("Error: Window index %d out of range (0..%d)"), Index, VisibleWindows.Num() - 1);
	}

	return TEXT("Error: Unknown action. Use 'list', 'select', or 'close'.");
}

// ── Wait tool ────────────────────────────────────────────────────────

bool USlateInspectorToolset::WaitFor(const FString& Text, const FString& TextGone)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	const bool bWaitForAppear = !Text.IsEmpty();
	const bool bWaitForDisappear = !TextGone.IsEmpty();

	if (!bWaitForAppear && !bWaitForDisappear)
	{
		return true;
	}

	// Walk the widget tree looking for text without rendering a full snapshot,
	// which would be unnecessary overhead for a simple text presence check.
	bool bFoundText = false;
	bool bFoundTextGone = false;

	TArray<TSharedRef<SWindow>> Windows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(Windows);

	TFunction<void(TSharedRef<SWidget>)> Walk = [&](TSharedRef<SWidget> Widget)
	{
		// Early out if both conditions already determined.
		if ((!bWaitForAppear || bFoundText) && (!bWaitForDisappear || bFoundTextGone))
		{
			return;
		}

		FString WidgetText;
		static const FName TextBlockType(TEXT("STextBlock"));
		if (Widget->GetType() == TextBlockType)
		{
			WidgetText = StaticCastSharedRef<STextBlock>(Widget)->GetText().ToString();
		}
#if WITH_ACCESSIBILITY
		if (WidgetText.IsEmpty())
		{
			WidgetText = Widget->GetAccessibleText(EAccessibleType::Main).ToString();
		}
#endif

		if (!WidgetText.IsEmpty())
		{
			if (bWaitForAppear && WidgetText.Contains(Text))
			{
				bFoundText = true;
			}
			if (bWaitForDisappear && WidgetText.Contains(TextGone))
			{
				bFoundTextGone = true;
			}
		}

		FChildren* Children = Widget->GetChildren();
		for (int32 Index = 0; Index < Children->Num(); ++Index)
		{
			Walk(Children->GetChildAt(Index));
		}
	};

	for (const TSharedRef<SWindow>& Window : Windows)
	{
		Walk(Window);
	}

	bool bAppearConditionMet = !bWaitForAppear || bFoundText;
	bool bDisappearConditionMet = !bWaitForDisappear || !bFoundTextGone;

	return bAppearConditionMet && bDisappearConditionMet;
}

// ── Form tool ────────────────────────────────────────────────────────

bool USlateInspectorToolset::FillForm(const TArray<FSlateInspectorToolsetFormField>& Fields)
{
	bool bAllSuccess = true;

	for (const FSlateInspectorToolsetFormField& Field : Fields)
	{
		bool bFieldSuccess = false;

		if (Field.FieldType.Equals(TEXT("textbox"), ESearchCase::IgnoreCase))
		{
			bFieldSuccess = Type(Field.Ref, Field.Value, false);
		}
		else if (Field.FieldType.Equals(TEXT("checkbox"), ESearchCase::IgnoreCase))
		{
			// Click the checkbox if the desired state differs from the current state.
			TSharedPtr<SWidget> Widget = ResolveRefOrWarn(Field.Ref);
			static const FName CheckBoxType(TEXT("SCheckBox"));
			if (Widget.IsValid() && Widget->GetType() == CheckBoxType)
			{
				TSharedRef<SCheckBox> CheckBox = StaticCastSharedRef<SCheckBox>(Widget.ToSharedRef());
				bool bDesiredChecked = Field.Value.Equals(TEXT("true"), ESearchCase::IgnoreCase);
				bool bCurrentlyChecked = CheckBox->IsChecked();
				if (bDesiredChecked != bCurrentlyChecked)
				{
					bFieldSuccess = Click(Field.Ref);
				}
				else
				{
					bFieldSuccess = true; // Already in desired state.
				}
			}
			else if (Widget.IsValid())
			{
				UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Widget '{Ref}' is not an SCheckBox", Field.Ref);
			}
		}
		else if (Field.FieldType.Equals(TEXT("combobox"), ESearchCase::IgnoreCase))
		{
			bFieldSuccess = SelectOption(Field.Ref, Field.Value);
		}
		else
		{
			UE_LOGFMT(LogSlateInspectorToolset, Warning, "SlateInspectorToolset: Unknown field type '{FieldType}'", Field.FieldType);
		}

		if (!bFieldSuccess)
		{
			bAllSuccess = false;
		}
	}

	return bAllSuccess;
}
