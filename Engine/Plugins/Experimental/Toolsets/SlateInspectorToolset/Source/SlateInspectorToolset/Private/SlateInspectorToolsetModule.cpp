// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolsetModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FSlateInspectorToolsetModule, SlateInspectorToolset)

void FSlateInspectorToolsetModule::StartupModule()
{
}

void FSlateInspectorToolsetModule::ShutdownModule()
{
}
