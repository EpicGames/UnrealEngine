// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolsetObserverManager.h"

#include "SlateInspectorToolsetRefCache.h"
#include "SlateInspectorToolsetSnapshotRenderer.h"

FSlateInspectorToolsetObserverManager& FSlateInspectorToolsetObserverManager::Get()
{
	static FSlateInspectorToolsetObserverManager Instance;
	return Instance;
}

FString FSlateInspectorToolsetObserverManager::AddObserver(TSharedPtr<SWidget> RootWidget, int32 MaxDepth)
{
	MaxDepth = FMath::Max(MaxDepth, 0);
	bool bRoot = !RootWidget.IsValid();

	// If an observer already exists for the same root, widen its depth (never shrink) and return its identifier.
	for (FSlateInspectorToolsetObserver& Existing : Observers)
	{
		if (bRoot && Existing.bRoot)
		{
			Existing.MaxDepth = FMath::Max(Existing.MaxDepth, MaxDepth);
			return Existing.Identifier;
		}
		if (!bRoot && !Existing.bRoot && Existing.RootWidget.Pin() == RootWidget)
		{
			Existing.MaxDepth = FMath::Max(Existing.MaxDepth, MaxDepth);
			return Existing.Identifier;
		}
	}

	FSlateInspectorToolsetObserver Observer;
	Observer.Identifier = FString::Printf(TEXT("observer_%d"), NextIdentifier++);
	Observer.bRoot = bRoot;
	Observer.RootWidget = RootWidget;
	Observer.MaxDepth = MaxDepth;

	Observers.Add(MoveTemp(Observer));

	return Observers.Last().Identifier;
}

bool FSlateInspectorToolsetObserverManager::RemoveObserver(const FString& Identifier)
{
	int32 Index = Observers.IndexOfByPredicate([&Identifier](const FSlateInspectorToolsetObserver& Observer)
	{
		return Observer.Identifier == Identifier;
	});

	if (Index == INDEX_NONE)
	{
		return false;
	}

	Observers.RemoveAt(Index);
	return true;
}

TArray<FSlateInspectorToolsetObserver> FSlateInspectorToolsetObserverManager::GetObservers() const
{
	return Observers;
}

FString FSlateInspectorToolsetObserverManager::GetCachedSnapshot(const FString& Identifier) const
{
	for (const FSlateInspectorToolsetObserver& Observer : Observers)
	{
		if (Observer.Identifier == Identifier)
		{
			return Observer.CachedSnapshotText;
		}
	}

	return FString();
}

FString FSlateInspectorToolsetObserverManager::FindMatchingObserverSnapshot(TSharedPtr<SWidget> RootWidget, int32 MaxDepth) const
{
	const FSlateInspectorToolsetObserver* BestMatch = nullptr;

	for (const FSlateInspectorToolsetObserver& Observer : Observers)
	{
		if (Observer.MaxDepth < MaxDepth)
		{
			continue;
		}

		if (!RootWidget.IsValid() && Observer.bRoot)
		{
			if (!BestMatch || Observer.MaxDepth > BestMatch->MaxDepth)
			{
				BestMatch = &Observer;
			}
		}
		else if (RootWidget.IsValid() && !Observer.bRoot)
		{
			TSharedPtr<SWidget> ObserverRoot = Observer.RootWidget.Pin();
			if (ObserverRoot == RootWidget)
			{
				if (!BestMatch || Observer.MaxDepth > BestMatch->MaxDepth)
				{
					BestMatch = &Observer;
				}
			}
		}
	}

	return BestMatch ? BestMatch->CachedSnapshotText : FString();
}

void FSlateInspectorToolsetObserverManager::Tick(float DeltaTime)
{
	AccumulatedTime += DeltaTime;

	if (AccumulatedTime < TickIntervalSeconds)
	{
		return;
	}
	AccumulatedTime = 0.0f;

	if (Observers.IsEmpty())
	{
		return;
	}

	// Remove non-root observers whose root widget has been destroyed.
	Observers.RemoveAll([](const FSlateInspectorToolsetObserver& Observer)
	{
		return !Observer.bRoot && !Observer.RootWidget.IsValid();
	});

	// Walk each observer's subtree and cache the snapshot text.
	for (FSlateInspectorToolsetObserver& Observer : Observers)
	{
		TSharedPtr<SWidget> Root = Observer.bRoot ? nullptr : Observer.RootWidget.Pin();
		Observer.CachedSnapshotText = FSlateInspectorToolsetSnapshotRenderer::Render(
			Root,
			Observer.MaxDepth,
			/* bIncludeSourceLocations */ false,
			/* bResetCache */ false);
	}

	// Periodically purge expired refs.
	++TickCounter;
	if (TickCounter >= PurgeEveryNTicks)
	{
		TickCounter = 0;
		FSlateInspectorToolsetRefCache::Get().PurgeExpired();
	}
}
