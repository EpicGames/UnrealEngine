// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolsetRefCache.h"

FSlateInspectorToolsetRefCache& FSlateInspectorToolsetRefCache::Get()
{
	static FSlateInspectorToolsetRefCache Instance;
	return Instance;
}

FString FSlateInspectorToolsetRefCache::GetOrAssignRef(TSharedRef<SWidget> Widget, const FString& RolePrefix)
{
	TWeakPtr<SWidget> WeakWidget = Widget;

	if (const FString* ExistingRef = WidgetToRef.Find(WeakWidget))
	{
		return *ExistingRef;
	}

	int32& Counter = RolePrefixCounters.FindOrAdd(RolePrefix, 0);
	++Counter;

	FString Ref = FString::Printf(TEXT("%s%d"), *RolePrefix, Counter);

	WidgetToRef.Add(WeakWidget, Ref);
	RefToWidget.Add(Ref, WeakWidget);

	return Ref;
}

FString FSlateInspectorToolsetRefCache::FindRef(TSharedRef<SWidget> Widget) const
{
	TWeakPtr<SWidget> WeakWidget = Widget;
	if (const FString* Ref = WidgetToRef.Find(WeakWidget))
	{
		return *Ref;
	}
	return FString();
}

TSharedPtr<SWidget> FSlateInspectorToolsetRefCache::ResolveRef(const FString& Ref) const
{
	if (const TWeakPtr<SWidget>* WeakWidget = RefToWidget.Find(Ref))
	{
		return WeakWidget->Pin();
	}
	return nullptr;
}

void FSlateInspectorToolsetRefCache::Reset()
{
	WidgetToRef.Empty();
	RefToWidget.Empty();
	RolePrefixCounters.Empty();
}

void FSlateInspectorToolsetRefCache::PurgeExpired()
{
	// Capture both the key and its ref string in a single pass.  Expired
	// weak pointers may hash identically, so a second Find() on the key
	// alone could silently fail or match the wrong entry.
	TArray<TPair<TWeakPtr<SWidget>, FString>> Expired;
	for (const auto& Pair : WidgetToRef)
	{
		if (!Pair.Key.IsValid())
		{
			Expired.Emplace(Pair.Key, Pair.Value);
		}
	}

	for (const auto& Entry : Expired)
	{
		RefToWidget.Remove(Entry.Value);
		WidgetToRef.Remove(Entry.Key);
	}
}
