// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolsetSnapshotRenderer.h"

#include "SlateInspectorToolsetRefCache.h"

#include "Framework/Application/SlateApplication.h"
#include "Misc/Paths.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

TMap<FName, FString> FSlateInspectorToolsetSnapshotRenderer::TypeToRole;
TArray<TPair<FString, FString>> FSlateInspectorToolsetSnapshotRenderer::TemplatePrefixToRole;
TMap<FString, FString> FSlateInspectorToolsetSnapshotRenderer::RoleToRefPrefix;
TSet<FName> FSlateInspectorToolsetSnapshotRenderer::StructuralContainers;
TMap<FName, TFunction<FString(TSharedRef<SWidget>)>> FSlateInspectorToolsetSnapshotRenderer::CustomLabelExtractors;
TMap<FName, TFunction<TArray<FString>(TSharedRef<SWidget>)>> FSlateInspectorToolsetSnapshotRenderer::CustomStateFlagsExtractors;
bool FSlateInspectorToolsetSnapshotRenderer::bMapsInitialized = false;

void FSlateInspectorToolsetSnapshotRenderer::EnsureMapsInitialized()
{
	if (bMapsInitialized)
	{
		return;
	}
	bMapsInitialized = true;

	// Type -> role mapping (non-template types use exact FName match).
	TypeToRole.Add(FName(TEXT("SWindow")),               TEXT("window"));
	TypeToRole.Add(FName(TEXT("SButton")),               TEXT("button"));
	TypeToRole.Add(FName(TEXT("SEditableText")),          TEXT("textbox"));
	TypeToRole.Add(FName(TEXT("SEditableTextBox")),       TEXT("textbox"));
	TypeToRole.Add(FName(TEXT("SMultiLineEditableText")), TEXT("textbox"));
	TypeToRole.Add(FName(TEXT("SCheckBox")),              TEXT("checkbox"));
	TypeToRole.Add(FName(TEXT("SComboButton")),           TEXT("combobox"));
	TypeToRole.Add(FName(TEXT("SSlider")),                TEXT("slider"));
	TypeToRole.Add(FName(TEXT("SMenuBar")),               TEXT("menubar"));
	TypeToRole.Add(FName(TEXT("SMenuAnchor")),            TEXT("menu"));
	TypeToRole.Add(FName(TEXT("SScrollBox")),             TEXT("scrollable"));
	TypeToRole.Add(FName(TEXT("SSplitter")),              TEXT("splitter"));
	TypeToRole.Add(FName(TEXT("SDockTab")),               TEXT("tab"));
	TypeToRole.Add(FName(TEXT("STabWell")),               TEXT("tablist"));
	TypeToRole.Add(FName(TEXT("SToolTip")),               TEXT("tooltip"));
	TypeToRole.Add(FName(TEXT("STextBlock")),             TEXT("text"));
	TypeToRole.Add(FName(TEXT("SRichTextBlock")),         TEXT("text"));
	TypeToRole.Add(FName(TEXT("SImage")),                 TEXT("image"));
	TypeToRole.Add(FName(TEXT("SProgressBar")),           TEXT("progressbar"));

	// Template types use prefix matching (e.g., "SComboBox<" matches "SComboBox<TSharedPtr<FString>>").
	TemplatePrefixToRole.Add(TPair<FString, FString>(TEXT("SComboBox<"),  TEXT("combobox")));
	TemplatePrefixToRole.Add(TPair<FString, FString>(TEXT("SSpinBox<"),   TEXT("slider")));
	TemplatePrefixToRole.Add(TPair<FString, FString>(TEXT("STreeView<"),  TEXT("treeview")));
	TemplatePrefixToRole.Add(TPair<FString, FString>(TEXT("SListView<"),  TEXT("listview")));
	TemplatePrefixToRole.Add(TPair<FString, FString>(TEXT("STableRow<"),  TEXT("listitem")));

	// Role -> ref prefix mapping.
	RoleToRefPrefix.Add(TEXT("window"),      TEXT("w"));
	RoleToRefPrefix.Add(TEXT("button"),      TEXT("b"));
	RoleToRefPrefix.Add(TEXT("textbox"),     TEXT("tb"));
	RoleToRefPrefix.Add(TEXT("checkbox"),    TEXT("cb"));
	RoleToRefPrefix.Add(TEXT("combobox"),    TEXT("co"));
	RoleToRefPrefix.Add(TEXT("slider"),      TEXT("sl"));
	RoleToRefPrefix.Add(TEXT("menubar"),     TEXT("mb"));
	RoleToRefPrefix.Add(TEXT("menu"),        TEXT("m"));
	RoleToRefPrefix.Add(TEXT("treeview"),    TEXT("tv"));
	RoleToRefPrefix.Add(TEXT("listview"),    TEXT("lv"));
	RoleToRefPrefix.Add(TEXT("listitem"),    TEXT("li"));
	RoleToRefPrefix.Add(TEXT("scrollable"),  TEXT("sc"));
	RoleToRefPrefix.Add(TEXT("splitter"),    TEXT("sp"));
	RoleToRefPrefix.Add(TEXT("tab"),         TEXT("t"));
	RoleToRefPrefix.Add(TEXT("tablist"),     TEXT("tl"));
	RoleToRefPrefix.Add(TEXT("tooltip"),     TEXT("tt"));
	RoleToRefPrefix.Add(TEXT("text"),        TEXT("x"));
	RoleToRefPrefix.Add(TEXT("image"),       TEXT("i"));
	RoleToRefPrefix.Add(TEXT("progressbar"), TEXT("pb"));

	// Structural containers to skip (promote children).
	StructuralContainers.Add(FName(TEXT("SBox")));
	StructuralContainers.Add(FName(TEXT("SBorder")));
	StructuralContainers.Add(FName(TEXT("SOverlay")));
	StructuralContainers.Add(FName(TEXT("SSpacer")));
	StructuralContainers.Add(FName(TEXT("SConstraintCanvas")));
	StructuralContainers.Add(FName(TEXT("SHorizontalBox")));
	StructuralContainers.Add(FName(TEXT("SVerticalBox")));
	StructuralContainers.Add(FName(TEXT("SGridPanel")));
	StructuralContainers.Add(FName(TEXT("SWrapBox")));
	StructuralContainers.Add(FName(TEXT("SWidgetSwitcher")));
	StructuralContainers.Add(FName(TEXT("SCanvas")));
	StructuralContainers.Add(FName(TEXT("SScaleBox")));
	StructuralContainers.Add(FName(TEXT("SSizeBox")));
	StructuralContainers.Add(FName(TEXT("SNullWidget")));
	StructuralContainers.Add(FName(TEXT("SInvalidationPanel")));
	StructuralContainers.Add(FName(TEXT("SRetainerWidget")));
}

void FSlateInspectorToolsetSnapshotRenderer::RegisterWidgetRole(const FName& WidgetType, const FString& Role, const FString& RefPrefix)
{
	EnsureMapsInitialized();
	TypeToRole.Add(WidgetType, Role);
	RoleToRefPrefix.Add(Role, RefPrefix);
}

void FSlateInspectorToolsetSnapshotRenderer::RegisterLabelExtractor(const FName& WidgetType, TFunction<FString(TSharedRef<SWidget>)> Extractor)
{
	EnsureMapsInitialized();
	CustomLabelExtractors.Add(WidgetType, MoveTemp(Extractor));
}

void FSlateInspectorToolsetSnapshotRenderer::RegisterStateFlagsExtractor(const FName& WidgetType, TFunction<TArray<FString>(TSharedRef<SWidget>)> Extractor)
{
	EnsureMapsInitialized();
	CustomStateFlagsExtractors.Add(WidgetType, MoveTemp(Extractor));
}

const FString* FSlateInspectorToolsetSnapshotRenderer::FindRoleForType(const FName& WidgetType, const FString& TypeString)
{
	// Try exact match first.
	if (const FString* Role = TypeToRole.Find(WidgetType))
	{
		return Role;
	}

	// Try prefix match for template types (e.g., "SComboBox<TSharedPtr<FString>>").
	for (const TPair<FString, FString>& Pair : TemplatePrefixToRole)
	{
		if (TypeString.StartsWith(Pair.Key))
		{
			return &Pair.Value;
		}
	}

	return nullptr;
}

FString FSlateInspectorToolsetSnapshotRenderer::Render(TSharedPtr<SWidget> Root, int32 MaxDepth, bool bIncludeSourceLocations, bool bResetCache)
{
	EnsureMapsInitialized();

	MaxDepth = FMath::Max(MaxDepth, 0);

	if (bResetCache)
	{
		FSlateInspectorToolsetRefCache::Get().Reset();
	}

	TStringBuilder<4096> Output;

	if (Root.IsValid())
	{
		RenderWidget(Output, Root.ToSharedRef(), 0, 0, MaxDepth, bIncludeSourceLocations);
	}
	else
	{
		// Snapshot all visible windows in z-order.
		if (FSlateApplication::IsInitialized())
		{
			TArray<TSharedRef<SWindow>> VisibleWindows;
			FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);
			for (const TSharedRef<SWindow>& Window : VisibleWindows)
			{
				RenderWidget(Output, Window, 0, 0, MaxDepth, bIncludeSourceLocations);
			}
		}
	}

	return Output.ToString();
}

void FSlateInspectorToolsetSnapshotRenderer::RenderWidget(
	FStringBuilderBase& Output,
	TSharedRef<SWidget> Widget,
	int32 IndentLevel,
	int32 CurrentDepth,
	int32 MaxDepth,
	bool bIncludeSourceLocations)
{
	if (CurrentDepth > MaxDepth)
	{
		return;
	}

	// Skip collapsed/hidden widgets.
	if (Widget->GetVisibility() == EVisibility::Collapsed || Widget->GetVisibility() == EVisibility::Hidden)
	{
		return;
	}

	const FName WidgetType = Widget->GetType();
	const FString TypeString = Widget->GetTypeAsString();

	// Check if this is a structural container (skip but recurse children at same indent).
	if (StructuralContainers.Contains(WidgetType))
	{
		FChildren* Children = Widget->GetChildren();
		for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
		{
			RenderWidget(Output, Children->GetChildAt(ChildIndex), IndentLevel, CurrentDepth + 1, MaxDepth, bIncludeSourceLocations);
		}
		return;
	}

	// Look up role for this widget type (handles both exact and template prefix match).
	const FString* Role = FindRoleForType(WidgetType, TypeString);

	// Text collapsing: if this widget has exactly one child and that child is STextBlock,
	// absorb the text child's label and don't recurse into it.
	bool bAbsorbedTextChild = false;
	FString AbsorbedText;
	FChildren* Children = Widget->GetChildren();
	if (Children->Num() == 1)
	{
		TSharedRef<SWidget> OnlyChild = Children->GetChildAt(0);
		static const FName TextBlockType(TEXT("STextBlock"));
		if (OnlyChild->GetType() == TextBlockType)
		{
			TSharedRef<STextBlock> TextBlock = StaticCastSharedRef<STextBlock>(OnlyChild);
			AbsorbedText = TextBlock->GetText().ToString();
			bAbsorbedTextChild = true;
		}
	}

	// If we have no role and no useful label, skip this widget but recurse children.
	if (!Role)
	{
		FString Label = ExtractLabel(Widget, TypeString);
		if (Label.IsEmpty() && !bAbsorbedTextChild)
		{
			for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
			{
				RenderWidget(Output, Children->GetChildAt(ChildIndex), IndentLevel, CurrentDepth + 1, MaxDepth, bIncludeSourceLocations);
			}
			return;
		}
	}

	// This widget is meaningful, emit it.
	FString Label;
	if (bAbsorbedTextChild && !AbsorbedText.IsEmpty())
	{
		Label = AbsorbedText;
	}
	else
	{
		Label = ExtractLabel(Widget, TypeString);
	}

	FString StateFlags = ExtractStateFlags(Widget, TypeString);

	// Determine the role string and ref prefix.
	FString RoleString = Role ? *Role : TEXT("generic");
	const FString* RefPrefix = RoleToRefPrefix.Find(RoleString);
	FString Prefix = RefPrefix ? *RefPrefix : TEXT("g");

	FString Ref = FSlateInspectorToolsetRefCache::Get().GetOrAssignRef(Widget, Prefix);

	// Emit the line.
	for (int32 Index = 0; Index < IndentLevel; ++Index)
	{
		Output.Append(TEXT("  "));
	}

	Output.Append(*RoleString);

	if (!Label.IsEmpty())
	{
		Output.Append(TEXT(" \""));
		// Escape quotes and control characters that would break the line-oriented format.
		for (TCHAR Character : Label)
		{
			switch (Character)
			{
			case TEXT('"'):  Output.Append(TEXT("\\\"")); break;
			case TEXT('\\'): Output.Append(TEXT("\\\\")); break;
			case TEXT('\n'): Output.Append(TEXT("\\n"));  break;
			case TEXT('\r'): Output.Append(TEXT("\\r"));  break;
			case TEXT('\t'): Output.Append(TEXT("\\t"));  break;
			default:         Output.AppendChar(Character); break;
			}
		}
		Output.Append(TEXT("\""));
	}

	if (!StateFlags.IsEmpty())
	{
		Output.Append(TEXT(" "));
		Output.Append(*StateFlags);
	}

	// Geometry: absolute position (desktop space) and local size (widget space).
	const FGeometry& Geometry = Widget->GetCachedGeometry();
	const FVector2D AbsolutePosition = Geometry.GetAbsolutePosition();
	const FVector2D LocalSize = Geometry.GetLocalSize();
	Output.Appendf(TEXT(" [pos=%.0f,%.0f size=%.0f,%.0f]"), AbsolutePosition.X, AbsolutePosition.Y, LocalSize.X, LocalSize.Y);

	if (bIncludeSourceLocations)
	{
#if !UE_BUILD_SHIPPING
		FName CreatedInLocation = Widget->GetCreatedInLocation();
		if (!CreatedInLocation.IsNone())
		{
			FString FilePath = CreatedInLocation.GetPlainNameString();
			int32 LineNumber = CreatedInLocation.GetNumber();
			FString FileName = FPaths::GetCleanFilename(FilePath);
			if (LineNumber > 0)
			{
				Output.Appendf(TEXT(" [src=%s:%d]"), *FileName, LineNumber);
			}
			else
			{
				Output.Appendf(TEXT(" [src=%s]"), *FileName);
			}
		}
#endif
	}

	Output.Append(TEXT(" [ref="));
	Output.Append(*Ref);
	Output.Append(TEXT("]\n"));

	// Recurse into children (unless we absorbed the only text child).
	if (!bAbsorbedTextChild)
	{
		for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
		{
			RenderWidget(Output, Children->GetChildAt(ChildIndex), IndentLevel + 1, CurrentDepth + 1, MaxDepth, bIncludeSourceLocations);
		}
	}
}

FString FSlateInspectorToolsetSnapshotRenderer::ExtractLabel(TSharedRef<SWidget> Widget, const FString& TypeString)
{
	// Prefer Slate's native accessible text since many widgets implement this already.
#if WITH_ACCESSIBILITY
	FText AccessibleText = Widget->GetAccessibleText(EAccessibleType::Main);
	if (!AccessibleText.IsEmpty())
	{
		return AccessibleText.ToString();
	}
#endif

	// Fall back to type-specific extraction when no accessible text is available.
	if (TypeString == TEXT("SWindow"))
	{
		return StaticCastSharedRef<SWindow>(Widget)->GetTitle().ToString();
	}
	if (TypeString == TEXT("STextBlock"))
	{
		return StaticCastSharedRef<STextBlock>(Widget)->GetText().ToString();
	}
	if (TypeString == TEXT("SEditableTextBox"))
	{
		return StaticCastSharedRef<SEditableTextBox>(Widget)->GetText().ToString();
	}
	if (TypeString == TEXT("SDockTab"))
	{
		return StaticCastSharedRef<SDockTab>(Widget)->GetTabLabel().ToString();
	}

	// Try custom label extractors.
	if (const auto* Extractor = CustomLabelExtractors.Find(Widget->GetType()))
	{
		return (*Extractor)(Widget);
	}

	return FString();
}

FString FSlateInspectorToolsetSnapshotRenderer::ExtractStateFlags(TSharedRef<SWidget> Widget, const FString& TypeString)
{
	TArray<FString> Flags;

	if (!Widget->IsEnabled())
	{
		Flags.Add(TEXT("disabled"));
	}

	if (Widget->HasKeyboardFocus())
	{
		Flags.Add(TEXT("focused"));
	}

	if (TypeString == TEXT("SCheckBox"))
	{
		ECheckBoxState CheckedState = StaticCastSharedRef<SCheckBox>(Widget)->GetCheckedState();
		switch (CheckedState)
		{
		case ECheckBoxState::Checked:
			Flags.Add(TEXT("checked"));
			break;
		case ECheckBoxState::Undetermined:
			Flags.Add(TEXT("indeterminate"));
			break;
		default:
			Flags.Add(TEXT("unchecked"));
			break;
		}
	}

	// Try custom state flags extractors.
	if (const auto* Extractor = CustomStateFlagsExtractors.Find(Widget->GetType()))
	{
		Flags.Append((*Extractor)(Widget));
	}

	if (Flags.IsEmpty())
	{
		return FString();
	}

	return FString::Printf(TEXT("[%s]"), *FString::Join(Flags, TEXT(", ")));
}
