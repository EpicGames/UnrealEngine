// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolsetSubsystem.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateInspectorToolsetObserverManager.h"
#include "SlateInspectorToolset.h"
#include "Subsystems/SubsystemCollection.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

static bool bEnableSlateInspectorToolset = true;

static void OnEnableSlateInspectorToolsetChanged(IConsoleVariable* Variable)
{
	if (GEditor == nullptr)
	{
		return;
	}
	
	if (USlateInspectorToolsetSubsystem* Subsystem = GEditor->GetEditorSubsystem<USlateInspectorToolsetSubsystem>())
	{
		Subsystem->SetToolsetEnabled(Variable->GetBool());
	}
}

static FAutoConsoleVariableRef CVarEnableSlateInspectorToolset(
	TEXT("SlateInspectorToolset.Enable"),
	bEnableSlateInspectorToolset,
	TEXT("Enable or disable SlateInspectorToolset registration. When disabled, its tools will not appear in the MCP tool list."),
	FConsoleVariableDelegate::CreateStatic(&OnEnableSlateInspectorToolsetChanged),
	ECVF_Default);

void USlateInspectorToolsetSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	SetToolsetEnabled(bEnableSlateInspectorToolset);

	if (FSlateApplication::IsInitialized())
	{
		PostTickDelegateHandle = FSlateApplication::Get().OnPostTick().AddUObject(this, &USlateInspectorToolsetSubsystem::OnSlatePostTick);
	}

	// Create a shallow default root observer (depth 0: window names only).
	// Use Observe() on specific windows or panels to drill deeper.
	RootObserverIdentifier = FSlateInspectorToolsetObserverManager::Get().AddObserver(nullptr, 0);
}

void USlateInspectorToolsetSubsystem::Deinitialize()
{
	if (!RootObserverIdentifier.IsEmpty())
	{
		FSlateInspectorToolsetObserverManager::Get().RemoveObserver(RootObserverIdentifier);
		RootObserverIdentifier.Empty();
	}

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnPostTick().Remove(PostTickDelegateHandle);
	}

	SetToolsetEnabled(false);
	Super::Deinitialize();
}

void USlateInspectorToolsetSubsystem::OnSlatePostTick(float DeltaTime)
{
	FSlateInspectorToolsetObserverManager::Get().Tick(DeltaTime);
}

void USlateInspectorToolsetSubsystem::SetToolsetEnabled(bool bEnabled)
{
	if (bEnabled && !bToolsetRegistered)
	{
		UToolsetRegistry::RegisterToolsetClass(USlateInspectorToolset::StaticClass());
		bToolsetRegistered = true;
	}
	else if (!bEnabled && bToolsetRegistered)
	{
		UToolsetRegistry::UnregisterToolsetClass(USlateInspectorToolset::StaticClass());
		bToolsetRegistered = false;
	}
}
