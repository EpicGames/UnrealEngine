// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Events.h"
#include "ToolsetRegistry/ToolsetDefinition.h"
#include "ToolsetRegistry/ToolsetImage.h"
#include "Widgets/SWidget.h"

#include "SlateInspectorToolset.generated.h"

/** Modifier keys held during an input simulation (click, drag, etc.). */
USTRUCT(BlueprintType)
struct FSlateInspectorToolsetModifierKeys
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	bool bShift = false;

	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	bool bCtrl = false;

	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	bool bAlt = false;

	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	bool bCmd = false;

	/** Convert to Slate's FModifierKeysState. */
	FModifierKeysState ToModifierKeysState() const
	{
		return FModifierKeysState(bShift, bShift, bCtrl, bCtrl, bAlt, bAlt, bCmd, bCmd, false);
	}
};

/** Describes a single form field for the FillForm tool. */
USTRUCT(BlueprintType)
struct FSlateInspectorToolsetFormField
{
	GENERATED_BODY()

	/** The ref identifier of the form field widget. */
	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	FString Ref;

	/** The value to set. */
	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	FString Value;

	/** The type of field: "textbox", "checkbox", or "combobox". */
	UPROPERTY(BlueprintReadWrite, Category = "SlateInspectorToolset")
	FString FieldType;
};

/**
 * Playwright-style Slate UI automation toolset.
 *
 * Exposes snapshot, screenshot, and interaction tools for driving the
 * Unreal Editor UI programmatically.  Registered via UToolsetRegistry
 * so the ModelContextProtocol plugin picks them up automatically.
 *
 * A shallow root observer (depth 0) continuously tracks top-level
 * windows.  Before working with a specific window or panel, call
 * Observe() on it to get deep widget coverage, then Unobserve() when
 * done.  Observers walk their subtree every ~100ms, assigning refs to
 * newly appeared widgets and keeping the ref cache current.
 *
 * Input simulation uses direct Slate event APIs (ProcessKeyCharEvent,
 * ProcessMouseButtonDownEvent, etc.) rather than the AutomationDriver,
 * because AutomationDriver's synchronous API deadlocks when called from
 * the game thread (which is where MCP tool calls execute).
 */
UCLASS(BlueprintType, MinimalAPI)
class USlateInspectorToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/** Capture a Slate UI accessibility snapshot.  Use this to read the current
	 * widget tree and discover refs for action tools (Click, Type, Hover, etc.).
	 * A shallow root observer (depth 0) covers top-level windows automatically.
	 * Before interacting with a specific window or panel, call Observe() on it
	 * to get deep coverage, then Snapshot that subtree to see its contents.
	 * Refs discovered by a previous Snapshot remain usable. You do NOT need to
	 * call Snapshot again before every action.
	 * @param Ref                      Subtree root ref. Empty = all windows.
	 * @param MaxDepth                 Maximum depth (default 30).
	 * @param bIncludeSourceLocations  Include [src=File:Line] tags showing where each widget was created in C++. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API FString Snapshot(const FString& Ref, int32 MaxDepth = 30, bool bIncludeSourceLocations = false);

	/** Register an observer on a widget subtree so its refs are continuously
	 * kept up to date (~100ms tick).  Call this on the window or panel you are
	 * about to work with. It ensures new widgets appearing in that subtree
	 * are assigned refs automatically.  Unobserve when you are done.
	 * A shallow root observer (depth 0) already covers top-level windows.
	 * @param Ref       Root widget ref to observe. Empty = all visible windows.
	 * @param MaxDepth  Maximum depth to walk from the root. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API FString Observe(const FString& Ref, int32 MaxDepth = 30);

	/** Remove an observer by its identifier.
	 * @param Identifier  The identifier returned by Observe(). */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool Unobserve(const FString& Identifier);

	/** List all active observers as a JSON array for debugging.
	 * Each entry includes the observer identifier, whether it is the root observer,
	 * the root widget ref (if any), max depth, and cached snapshot size. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API FString ListObservers();

	/** Screenshot a Slate widget or the active editor window. Prefer this over
	 * SceneTools.take_screenshot for Editor UI; use SceneTools only for 3D viewport.
	 * @param Ref  Slate widget ref. Empty = active window. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API FToolsetImage Screenshot(const FString& Ref);

	// -- Action tools --

	/** Click a Slate widget identified by its ref.
	 * @param Ref         Slate widget ref.
	 * @param Button      "left", "right", or "middle".
	 * @param DoubleClick True for double-click.
	 * @param Modifiers   Modifier keys held during the click. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool Click(const FString& Ref, const FString& Button = TEXT("left"), bool DoubleClick = false, const FSlateInspectorToolsetModifierKeys& Modifiers = FSlateInspectorToolsetModifierKeys());

	/** Hover over a Slate widget, triggering any hover state or tooltip.
	 * @param Ref  Slate widget ref. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool Hover(const FString& Ref);

	/** Type text into a Slate text input widget. Focuses the widget first,
	 * then sends one key event per character.
	 * @param Ref    Slate textbox ref.
	 * @param Text   Text to type.
	 * @param Submit Press Enter after typing. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool Type(const FString& Ref, const FString& Text, bool Submit = false);

	/** Press and release a keyboard key on the currently focused Slate widget.
	 * Supports modifier prefixes: "Ctrl+C", "Shift+1".
	 * @param Key  Key name with optional modifiers, e.g. "Enter", "Ctrl+A", "Shift+Tab". */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool PressKey(const FString& Key);

	/** Select an option in a Slate combobox by its text label.
	 * Opens the dropdown, finds the matching text, and clicks it.
	 * @param Ref    Slate combobox ref.
	 * @param Value  Exact option text to select. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool SelectOption(const FString& Ref, const FString& Value);

	/** Drag from one Slate widget to another (mouse down, move, release).
	 * @param StartRef   Slate widget ref for the drag source.
	 * @param EndRef     Slate widget ref for the drop target.
	 * @param Modifiers  Modifier keys held during the drag. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool Drag(const FString& StartRef, const FString& EndRef, const FSlateInspectorToolsetModifierKeys& Modifiers = FSlateInspectorToolsetModifierKeys());

	/** List, select, or close top-level Slate editor windows.
	 * @param Action  "list" returns JSON array, "select" brings to front, "close" destroys.
	 * @param Index   Window index for select/close. */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API FString Windows(const FString& Action = TEXT("list"), int32 Index = -1);

	/** Check if text is present or absent in the Slate widget tree.
	 * Non-blocking: checks once and returns immediately. Poll to wait.
	 * @param Text       Text that must be present (empty = skip).
	 * @param TextGone   Text that must be absent (empty = skip). */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool WaitFor(const FString& Text = TEXT(""), const FString& TextGone = TEXT(""));

	/** Fill multiple Slate form fields at once.
	 * @param Fields  Array of {Ref, Value, FieldType} where FieldType is
	 *                "textbox", "checkbox", or "combobox". */
	UFUNCTION(meta = (AICallable), Category = "SlateInspectorToolset")
	static SLATEINSPECTORTOOLSET_API bool FillForm(const TArray<FSlateInspectorToolsetFormField>& Fields);

private:

	/** Resolve a ref to a widget, logging a warning if it fails. */
	static TSharedPtr<SWidget> ResolveRefOrWarn(const FString& Ref);

	/** Get the absolute screen-space center of a widget. */
	static FVector2D GetWidgetScreenCenter(TSharedRef<SWidget> Widget);

	/** Map a button name ("left", "right", "middle") to an FKey. */
	static FKey ParseMouseButtonKey(const FString& Button);

	/** Simulate a mouse click at a widget's screen-space center. */
	static bool SimulateClick(TSharedRef<SWidget> Widget, FKey MouseButton, bool bDoubleClick, const FModifierKeysState& ModifierKeys = FModifierKeysState());
};
