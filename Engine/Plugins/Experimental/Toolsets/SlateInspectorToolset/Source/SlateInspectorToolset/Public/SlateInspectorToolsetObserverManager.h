// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/** A single observer watching a Slate widget subtree. */
struct FSlateInspectorToolsetObserver
{
	/** Unique identifier returned by AddObserver(). */
	FString Identifier;

	/** Root widget to observe. Invalid (null) when bRoot is true. */
	TWeakPtr<SWidget> RootWidget;

	/** When true, this is the root observer covering all visible windows (RootWidget is ignored). */
	bool bRoot = false;

	/** Maximum depth to walk from the root. */
	int32 MaxDepth = 30;

	/** Cached snapshot text from the most recent observer tick. */
	FString CachedSnapshotText;
};

/**
 * Manages a set of observers that continuously walk Slate widget subtrees
 * on a throttled tick, keeping the ref cache up to date as the UI changes.
 *
 * A default root observer (all visible windows) is always present.
 */
class FSlateInspectorToolsetObserverManager
{
public:

	SLATEINSPECTORTOOLSET_API static FSlateInspectorToolsetObserverManager& Get();

	/**
	 * Register an observer on a widget subtree.
	 *
	 * @param RootWidget  The root of the subtree to observe.  Null = all visible windows.
	 * @param MaxDepth    Maximum depth to walk.
	 * @return The observer identifier (used with RemoveObserver and GetCachedSnapshot).
	 */
	SLATEINSPECTORTOOLSET_API FString AddObserver(TSharedPtr<SWidget> RootWidget, int32 MaxDepth = 30);

	/**
	 * Remove an observer by identifier.
	 *
	 * @return True if the observer was found and removed.
	 */
	SLATEINSPECTORTOOLSET_API bool RemoveObserver(const FString& Identifier);

	/** Returns a copy of all active observers. */
	SLATEINSPECTORTOOLSET_API TArray<FSlateInspectorToolsetObserver> GetObservers() const;

	/**
	 * Returns the cached snapshot text for an observer.
	 *
	 * @return The most recently rendered snapshot text, or empty if not found.
	 */
	SLATEINSPECTORTOOLSET_API FString GetCachedSnapshot(const FString& Identifier) const;

	/**
	 * Find the best matching observer for the given root and return its cached snapshot.
	 *
	 * A null RootWidget matches the root observer.  Otherwise, matches
	 * an observer whose root is the given widget (preferring deeper MaxDepth).
	 *
	 * @param RootWidget  The root to match. Null = root observer.
	 * @param MaxDepth    Only match observers with at least this depth.
	 * @return The cached snapshot text, or empty if no matching observer found.
	 */
	SLATEINSPECTORTOOLSET_API FString FindMatchingObserverSnapshot(TSharedPtr<SWidget> RootWidget, int32 MaxDepth) const;

	/**
	 * Called each Slate post-tick.  Throttled internally to ~100ms intervals.
	 */
	SLATEINSPECTORTOOLSET_API void Tick(float DeltaTime);

private:

	TArray<FSlateInspectorToolsetObserver> Observers;

	int32 NextIdentifier = 1;

	/** Accumulated time since last observer pass. */
	float AccumulatedTime = 0.0f;

	/** Tick counter for periodic purge scheduling. */
	int32 TickCounter = 0;

	static constexpr float TickIntervalSeconds = 0.1f;
	static constexpr int32 PurgeEveryNTicks = 10;
};
