// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/**
 * Manages bidirectional mapping between SWidget instances and short reference
 * strings used by the SlateInspectorToolset snapshot and action tools.
 *
 * # Observer model (append-only)
 *
 * When observers are active, refs are never reset.  New widgets encountered
 * during observer ticks receive the next available counter value for their
 * role prefix.  Existing widgets keep their refs indefinitely (the same
 * SWidget pointer always maps to the same ref string).  Destroyed widgets
 * are cleaned up periodically via PurgeExpired().
 *
 * Refs are no longer dense after widgets are destroyed (you may see b1, b3, b7
 * instead of b1, b2, b3), but keeping refs alive for existing widgets
 * matters more than compactness.
 *
 * Reset() is available for callers that need to clear the cache entirely
 * and start fresh (e.g. testing).
 */
class FSlateInspectorToolsetRefCache
{
public:

	SLATEINSPECTORTOOLSET_API static FSlateInspectorToolsetRefCache& Get();

	/**
	 * Returns existing ref or assigns a new one using the role prefix.
	 *
	 * The role prefix determines the ref format: "b" for buttons produces
	 * b1, b2, b3, etc.  Within a single snapshot pass, the same widget
	 * always gets the same ref (deduplication via WidgetToRef lookup).
	 *
	 * @param Widget      The widget to assign a ref to.
	 * @param RolePrefix  Short prefix derived from the widget's role (e.g., "b", "tb", "cb").
	 * @return The assigned ref string (e.g., "b3").
	 */
	SLATEINSPECTORTOOLSET_API FString GetOrAssignRef(TSharedRef<SWidget> Widget, const FString& RolePrefix);

	/**
	 * Resolves a ref string back to a live widget.
	 *
	 * Returns nullptr if the ref is unknown or the widget has been destroyed
	 * since the last snapshot.
	 */
	SLATEINSPECTORTOOLSET_API TSharedPtr<SWidget> ResolveRef(const FString& Ref) const;

	/**
	 * Looks up the ref for a widget without assigning one.
	 *
	 * @return The ref string, or empty if the widget has no ref.
	 */
	SLATEINSPECTORTOOLSET_API FString FindRef(TSharedRef<SWidget> Widget) const;

	/**
	 * Clears all mappings and resets counters to zero.
	 * Observer-driven passes should NOT call this because they rely on
	 * the append-only behavior to preserve existing refs.
	 */
	SLATEINSPECTORTOOLSET_API void Reset();

	/**
	 * Removes entries whose widget has been destroyed (weak pointer expired).
	 *
	 * Called periodically by the observer manager to prevent unbounded
	 * map growth.  Safe to call at any time.
	 */
	SLATEINSPECTORTOOLSET_API void PurgeExpired();

private:

	/** Widget -> ref string (deduplication within a single snapshot pass). */
	TMap<TWeakPtr<SWidget>, FString> WidgetToRef;

	/** Ref string -> widget (reverse lookup for action tools). */
	TMap<FString, TWeakPtr<SWidget>> RefToWidget;

	/** Per-role-prefix counters for generating sequential ref numbers. */
	TMap<FString, int32> RolePrefixCounters;
};
