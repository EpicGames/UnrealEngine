// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/**
 * Renders a text-based accessibility snapshot of a Slate widget tree.
 *
 * Walks the tree depth-first, emitting one line per meaningful widget with
 * role, label, state flags, and a compact ref identifier.  Structural
 * containers (SBox, SBorder, SOverlay, etc.) are elided and their children
 * are emitted at the parent's indent level.
 */
class FSlateInspectorToolsetSnapshotRenderer
{
public:

	/**
	 * Render a snapshot starting from the given root widget.
	 *
	 * @param Root                     The widget subtree to snapshot.  If null, snapshots all top-level windows.
	 * @param MaxDepth                 Maximum recursion depth (default 30).
	 * @param bIncludeSourceLocations  Include [src=File:Line] tags showing where each widget was created in C++.
	 * @param bResetCache              When true (default), clears the ref cache before rendering.
	 *                                 Pass false for observer-driven passes that preserve existing refs.
	 * @return Indented text representation of the widget tree.
	 */
	SLATEINSPECTORTOOLSET_API static FString Render(TSharedPtr<SWidget> Root, int32 MaxDepth = 30, bool bIncludeSourceLocations = false, bool bResetCache = true);

	/** Register a custom widget type so it appears in snapshots. */
	SLATEINSPECTORTOOLSET_API static void RegisterWidgetRole(const FName& WidgetType, const FString& Role, const FString& RefPrefix);

	/** Register a custom label extractor for a widget type. */
	SLATEINSPECTORTOOLSET_API static void RegisterLabelExtractor(const FName& WidgetType, TFunction<FString(TSharedRef<SWidget>)> Extractor);

	/** Register a custom state flags extractor for a widget type. */
	SLATEINSPECTORTOOLSET_API static void RegisterStateFlagsExtractor(const FName& WidgetType, TFunction<TArray<FString>(TSharedRef<SWidget>)> Extractor);

private:

	/** Initializes the type-to-role and structural-skip maps on first use. */
	static void EnsureMapsInitialized();

	/** Recursive depth-first walk. */
	static void RenderWidget(
		FStringBuilderBase& Output,
		TSharedRef<SWidget> Widget,
		int32 IndentLevel,
		int32 CurrentDepth,
		int32 MaxDepth,
		bool bIncludeSourceLocations);

	/** Extract a human-readable label from a widget via type-specific methods. */
	static FString ExtractLabel(TSharedRef<SWidget> Widget, const FString& TypeString);

	/** Extract state flags (visibility, enabled, checked, focused). */
	static FString ExtractStateFlags(TSharedRef<SWidget> Widget, const FString& TypeString);

	/** Look up role for a widget type, handling template types via prefix matching. */
	static const FString* FindRoleForType(const FName& WidgetType, const FString& TypeString);

	/** Map from widget type name to role string (e.g., "SButton" -> "button"). */
	static TMap<FName, FString> TypeToRole;

	/** Prefix-based role mapping for template types (e.g., "SComboBox" matches "SComboBox<...>"). */
	static TArray<TPair<FString, FString>> TemplatePrefixToRole;

	/** Map from role string to ref prefix (e.g., "button" -> "b"). */
	static TMap<FString, FString> RoleToRefPrefix;

	/** Set of structural container type names that should be skipped (children promoted). */
	static TSet<FName> StructuralContainers;

	/** Custom label extractors registered via RegisterLabelExtractor. */
	static TMap<FName, TFunction<FString(TSharedRef<SWidget>)>> CustomLabelExtractors;

	/** Custom state flags extractors registered via RegisterStateFlagsExtractor. */
	static TMap<FName, TFunction<TArray<FString>(TSharedRef<SWidget>)>> CustomStateFlagsExtractors;

	static bool bMapsInitialized;
};
