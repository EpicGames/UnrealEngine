// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "SlateInspectorToolsetSubsystem.generated.h"

class FSubsystemCollectionBase;

UCLASS(MinimalAPI)
class USlateInspectorToolsetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	virtual void Deinitialize() override;

	/** Register or unregister the toolset at runtime. */
	void SetToolsetEnabled(bool bEnabled);

private:

	/** Forwarded from FSlateApplication::OnPostTick to drive observer updates. */
	void OnSlatePostTick(float DeltaTime);

	bool bToolsetRegistered = false;

	FDelegateHandle PostTickDelegateHandle;

	FString RootObserverIdentifier;
};
