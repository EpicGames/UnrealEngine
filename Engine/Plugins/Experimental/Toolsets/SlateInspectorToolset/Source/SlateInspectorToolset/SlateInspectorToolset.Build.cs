// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SlateInspectorToolset : ModuleRules
	{
		public SlateInspectorToolset(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"ToolsetRegistry",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"EditorSubsystem",
					"Engine",
					"ImageWrapper",
					"InputCore",
					"Slate",
					"SlateCore",
					"UnrealEd",
				}
			);
		}
	}
}
