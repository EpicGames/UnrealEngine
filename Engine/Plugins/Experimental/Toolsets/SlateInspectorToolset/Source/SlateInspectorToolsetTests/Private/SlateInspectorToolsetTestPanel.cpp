// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateInspectorToolsetTestPanel.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SSlateInspectorToolsetTestPanel::Construct(const FArguments& InArgs)
{
	ComboOptions.Add(MakeShared<FString>(TEXT("Alpha")));
	ComboOptions.Add(MakeShared<FString>(TEXT("Beta")));
	ComboOptions.Add(MakeShared<FString>(TEXT("Gamma")));
	SelectedComboOption = ComboOptions[0];

	ChildSlot
	[
		SNew(SVerticalBox)

		// Primary button
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SButton)
			.Text(FText::FromString(TEXT("Test Button")))
			.OnClicked_Lambda([this]()
			{
				++ButtonClickCount;
				return FReply::Handled();
			})
		]

		// Second button (for double-click testing)
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SButton)
			.Text(FText::FromString(TEXT("Second Button")))
			.OnClicked_Lambda([this]()
			{
				++SecondButtonClickCount;
				return FReply::Handled();
			})
		]

		// Single-line text box
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SAssignNew(TextBox, SEditableTextBox)
			.HintText(FText::FromString(TEXT("Type here")))
		]

		// Multi-line text box
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(SBox)
			.HeightOverride(60)
			[
				SAssignNew(MultiLineTextBox, SMultiLineEditableTextBox)
				.HintText(FText::FromString(TEXT("Multi-line")))
			]
		]

		// Search box
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SAssignNew(SearchBox, SSearchBox)
		]

		// Checkbox
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SAssignNew(CheckBox, SCheckBox)
		]

		// Combobox
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SAssignNew(ComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&ComboOptions)
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NewValue, ESelectInfo::Type)
			{
				SelectedComboOption = NewValue;
			})
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
			{
				return SNew(STextBlock).Text(FText::FromString(*Item));
			})
			[
				SNew(STextBlock)
				.Text_Lambda([this]() { return FText::FromString(*SelectedComboOption); })
			]
		]

		// Slider
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SAssignNew(Slider, SSlider)
			.Value(0.0f)
			.OnValueChanged_Lambda([this](float NewValue)
			{
				SliderValue = NewValue;
			})
		]

		// SpinBox
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SAssignNew(SpinBox, SSpinBox<float>)
			.MinValue(0.0f)
			.MaxValue(100.0f)
			.Value(0.0f)
			.OnValueChanged_Lambda([this](float NewValue)
			{
				SpinBoxValue = NewValue;
			})
		]

		// Static text label
		+ SVerticalBox::Slot().AutoHeight().Padding(4)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Static Label")))
		]
	];
}
