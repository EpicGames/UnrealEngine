// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

/**
 * A widget panel containing one of each common widget type.
 * Used by SlateInspectorToolset automation tests to exercise snapshot, observer,
 * and input simulation tools against known widgets.
 */
class SSlateInspectorToolsetTestPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSlateInspectorToolsetTestPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// Button
	int32 ButtonClickCount = 0;
	int32 SecondButtonClickCount = 0;

	// Text input
	TSharedPtr<SEditableTextBox> TextBox;
	TSharedPtr<SMultiLineEditableTextBox> MultiLineTextBox;
	TSharedPtr<SSearchBox> SearchBox;

	// Toggle
	TSharedPtr<SCheckBox> CheckBox;

	// Combobox
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ComboBox;
	TArray<TSharedPtr<FString>> ComboOptions;
	TSharedPtr<FString> SelectedComboOption;

	// Slider / SpinBox
	TSharedPtr<SSlider> Slider;
	float SliderValue = 0.0f;
	TSharedPtr<SSpinBox<float>> SpinBox;
	float SpinBoxValue = 0.0f;
};
