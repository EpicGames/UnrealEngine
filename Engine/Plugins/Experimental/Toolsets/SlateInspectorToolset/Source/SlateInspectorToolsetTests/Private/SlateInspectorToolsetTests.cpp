// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Components/CQTestSlateComponent.h"
#include "SlateInspectorToolsetTestPanel.h"

#include "Framework/Application/SlateApplication.h"
#include "SlateInspectorToolsetObserverManager.h"
#include "SlateInspectorToolsetRefCache.h"
#include "SlateInspectorToolset.h"
#include "Widgets/SWindow.h"

TEST_CLASS(SlateInspectorToolsetTest, "AI.Toolsets.SlateInspectorToolset")
{
	TUniquePtr<FCQTestSlateComponent> SlateComponent;
	TSharedPtr<SWindow> TestWindow;
	TSharedPtr<SSlateInspectorToolsetTestPanel> TestPanel;

	BEFORE_EACH()
	{
		SlateComponent = MakeUnique<FCQTestSlateComponent>();

		TestPanel = SNew(SSlateInspectorToolsetTestPanel);

		TestWindow = SNew(SWindow)
			.Title(FText::FromString(TEXT("SlateInspectorToolset Test Window")))
			.ScreenPosition(FVector2D(100, 100))
			.ClientSize(FVector2D(400, 600))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.FocusWhenFirstShown(true)
			[
				TestPanel.ToSharedRef()
			];

		FSlateApplication::Get().AddWindow(TestWindow.ToSharedRef());
		TestWindow->MoveWindowTo(FVector2D(100, 100));
		TestWindow->BringToFront();

		// Force layout so widgets have valid geometry before any interaction.
		FSlateApplication::Get().Tick();
	}

	AFTER_EACH()
	{
		// Clean up any observers created during the test (except the root observer).
		for (const FSlateInspectorToolsetObserver& Observer : FSlateInspectorToolsetObserverManager::Get().GetObservers())
		{
			if (!Observer.bRoot)
			{
				FSlateInspectorToolsetObserverManager::Get().RemoveObserver(Observer.Identifier);
			}
		}

		if (TestWindow.IsValid())
		{
			TestWindow->RequestDestroyWindow();
			TestWindow.Reset();
		}
		TestPanel.Reset();
		SlateComponent.Reset();
	}

	// ── Snapshot tests ──────────────────────────────────────────────

	TEST_METHOD(Snapshot_ReturnsWidgetTree)
	{
		const FString Snapshot = SnapshotTestWindow();
		ASSERT_THAT(IsTrue(!Snapshot.IsEmpty()));
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("button"))));
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("textbox"))));
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("checkbox"))));
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("combobox"))));
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("slider"))));
	}

	TEST_METHOD(Snapshot_SubtreeByRef)
	{
		const FString FullSnapshot = USlateInspectorToolset::Snapshot(TEXT(""), 30);
		ASSERT_THAT(IsTrue(FullSnapshot.Contains(TEXT("SlateInspectorToolset Test Window"))));

		const FString WindowRef = FindRefByRole(FullSnapshot, TEXT("window"), TEXT("SlateInspectorToolset Test Window"));
		ASSERT_THAT(IsTrue(!WindowRef.IsEmpty()));

		const FString SubtreeSnapshot = USlateInspectorToolset::Snapshot(WindowRef, 30);
		ASSERT_THAT(IsTrue(SubtreeSnapshot.Contains(TEXT("button"))));
	}

	TEST_METHOD(Snapshot_ContainsStaticLabel)
	{
		const FString Snapshot = SnapshotTestWindow();
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("Static Label"))));
	}

	TEST_METHOD(Snapshot_ContainsSpinBox)
	{
		const FString Snapshot = SnapshotTestWindow();
		// SSpinBox<> maps to "slider" role via template prefix matching.
		ASSERT_THAT(IsTrue(Snapshot.Contains(TEXT("slider"))));
	}

	// ── Observer tests ──────────────────────────────────────────────

	TEST_METHOD(Observe_CreatesObserver)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString WindowRef = FindRefByRole(Snapshot, TEXT("window"), TEXT("SlateInspectorToolset Test Window"));
		ASSERT_THAT(IsTrue(!WindowRef.IsEmpty()));

		const FString Identifier = USlateInspectorToolset::Observe(WindowRef, 30);
		ASSERT_THAT(IsTrue(!Identifier.IsEmpty()));
		ASSERT_THAT(IsTrue(!Identifier.StartsWith(TEXT("Error"))));

		const FString ObserverList = USlateInspectorToolset::ListObservers();
		ASSERT_THAT(IsTrue(ObserverList.Contains(Identifier)));

		USlateInspectorToolset::Unobserve(Identifier);
	}

	TEST_METHOD(Observe_Deduplicates)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString WindowRef = FindRefByRole(Snapshot, TEXT("window"), TEXT("SlateInspectorToolset Test Window"));

		const FString Identifier1 = USlateInspectorToolset::Observe(WindowRef, 30);
		const FString Identifier2 = USlateInspectorToolset::Observe(WindowRef, 30);
		ASSERT_THAT(IsTrue(Identifier1 == Identifier2));

		USlateInspectorToolset::Unobserve(Identifier1);
	}

	TEST_METHOD(Unobserve_RemovesObserver)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString WindowRef = FindRefByRole(Snapshot, TEXT("window"), TEXT("SlateInspectorToolset Test Window"));
		const FString Identifier = USlateInspectorToolset::Observe(WindowRef, 30);

		const bool bRemoved = USlateInspectorToolset::Unobserve(Identifier);
		ASSERT_THAT(IsTrue(bRemoved));

		const FString ObserverList = USlateInspectorToolset::ListObservers();
		ASSERT_THAT(IsTrue(!ObserverList.Contains(Identifier)));
	}

	TEST_METHOD(Unobserve_ReturnsFalseForUnknown)
	{
		const bool bRemoved = USlateInspectorToolset::Unobserve(TEXT("nonexistent_observer"));
		ASSERT_THAT(IsTrue(!bRemoved));
	}

	TEST_METHOD(Observer_KeepsRefsAlive)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
		ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

		const FString WindowRef = FindRefByRole(Snapshot, TEXT("window"), TEXT("SlateInspectorToolset Test Window"));
		const FString ObserverId = USlateInspectorToolset::Observe(WindowRef, 30);

		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(15); })
			.Then([this, ButtonRef, ObserverId]()
			{
				const bool bClicked = USlateInspectorToolset::Click(ButtonRef);
				ASSERT_THAT(IsTrue(bClicked));
				ASSERT_THAT(IsTrue(TestPanel->ButtonClickCount > 0));
				USlateInspectorToolset::Unobserve(ObserverId);
			});
	}

	// ── Click tests ─────────────────────────────────────────────────

	TEST_METHOD(Click_PressesButton)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				ASSERT_THAT(IsTrue(TestPanel->ButtonClickCount == 0));
				const bool bClicked = USlateInspectorToolset::Click(ButtonRef);
				ASSERT_THAT(IsTrue(bClicked));
				ASSERT_THAT(IsTrue(TestPanel->ButtonClickCount == 1));
			});
	}

	TEST_METHOD(Click_MultipleButtons)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString FirstRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				const FString SecondRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Second Button"));
				ASSERT_THAT(IsTrue(!FirstRef.IsEmpty()));
				ASSERT_THAT(IsTrue(!SecondRef.IsEmpty()));
				ASSERT_THAT(IsTrue(FirstRef != SecondRef));

				USlateInspectorToolset::Click(FirstRef);
				USlateInspectorToolset::Click(SecondRef);
				USlateInspectorToolset::Click(FirstRef);

				ASSERT_THAT(IsTrue(TestPanel->ButtonClickCount == 2));
				ASSERT_THAT(IsTrue(TestPanel->SecondButtonClickCount == 1));
			});
	}

	TEST_METHOD(Click_DoubleClick)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				const bool bClicked = USlateInspectorToolset::Click(ButtonRef, TEXT("left"), true);
				ASSERT_THAT(IsTrue(bClicked));
			});
	}

	TEST_METHOD(Click_RightClick)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				const bool bClicked = USlateInspectorToolset::Click(ButtonRef, TEXT("right"));
				ASSERT_THAT(IsTrue(bClicked));
			});
	}

	TEST_METHOD(Click_InvalidRef)
	{
		const bool bClicked = USlateInspectorToolset::Click(TEXT("nonexistent_ref"));
		ASSERT_THAT(IsTrue(!bClicked));
	}

	TEST_METHOD(Click_WithShiftModifier)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				FSlateInspectorToolsetModifierKeys Modifiers;
				Modifiers.bShift = true;
				const bool bClicked = USlateInspectorToolset::Click(ButtonRef, TEXT("left"), false, Modifiers);
				ASSERT_THAT(IsTrue(bClicked));
				ASSERT_THAT(IsTrue(TestPanel->ButtonClickCount == 1));
			});
	}

	TEST_METHOD(Click_WithCtrlModifier)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				FSlateInspectorToolsetModifierKeys Modifiers;
				Modifiers.bCtrl = true;
				const bool bClicked = USlateInspectorToolset::Click(ButtonRef, TEXT("left"), false, Modifiers);
				ASSERT_THAT(IsTrue(bClicked));
			});
	}

	TEST_METHOD(Click_WithMultipleModifiers)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				FSlateInspectorToolsetModifierKeys Modifiers;
				Modifiers.bShift = true;
				Modifiers.bCtrl = true;
				Modifiers.bAlt = true;
				const bool bClicked = USlateInspectorToolset::Click(ButtonRef, TEXT("left"), false, Modifiers);
				ASSERT_THAT(IsTrue(bClicked));
			});
	}

	// ── Drag tests ─────────────────────────────────────────────────

	TEST_METHOD(Drag_BetweenButtons)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString FirstRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				const FString SecondRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Second Button"));
				ASSERT_THAT(IsTrue(!FirstRef.IsEmpty()));
				ASSERT_THAT(IsTrue(!SecondRef.IsEmpty()));

				const bool bDragged = USlateInspectorToolset::Drag(FirstRef, SecondRef);
				ASSERT_THAT(IsTrue(bDragged));
			});
	}

	TEST_METHOD(Drag_WithModifiers)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString FirstRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				const FString SecondRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Second Button"));

				FSlateInspectorToolsetModifierKeys Modifiers;
				Modifiers.bShift = true;
				const bool bDragged = USlateInspectorToolset::Drag(FirstRef, SecondRef, Modifiers);
				ASSERT_THAT(IsTrue(bDragged));
			});
	}

	TEST_METHOD(Drag_InvalidStartRef)
	{
		const bool bDragged = USlateInspectorToolset::Drag(TEXT("nonexistent"), TEXT("nonexistent2"));
		ASSERT_THAT(IsTrue(!bDragged));
	}

	TEST_METHOD(Drag_SliderChangesValue)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				ASSERT_THAT(IsTrue(TestPanel->SliderValue == 0.0f));

				const FString Snapshot = SnapshotTestWindow();
				const TArray<FString> SliderRefs = FindAllRefsByRole(Snapshot, TEXT("slider"));
				// First slider ref is the SSlider, second is the SSpinBox.
				ASSERT_THAT(IsTrue(SliderRefs.Num() >= 1));
				const FString SliderRef = SliderRefs[0];

				// Drag from the slider to one of the buttons (to the right and up).
				// This should move the slider thumb and change the value.
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				const bool bDragged = USlateInspectorToolset::Drag(SliderRef, ButtonRef);
				ASSERT_THAT(IsTrue(bDragged));

				// The slider value should have changed from its initial 0.0.
				// We can't predict the exact value since it depends on geometry,
				// but it should no longer be zero.
				ASSERT_THAT(IsTrue(TestPanel->SliderValue != 0.0f));
			});
	}

	// ── Checkbox tests ──────────────────────────────────────────────

	TEST_METHOD(Click_TogglesCheckbox)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString CheckBoxRef = FindRefByRole(Snapshot, TEXT("checkbox"), TEXT(""));
				ASSERT_THAT(IsTrue(!CheckBoxRef.IsEmpty()));

				ASSERT_THAT(IsTrue(!TestPanel->CheckBox->IsChecked()));

				USlateInspectorToolset::Click(CheckBoxRef);
				ASSERT_THAT(IsTrue(TestPanel->CheckBox->IsChecked()));

				USlateInspectorToolset::Click(CheckBoxRef);
				ASSERT_THAT(IsTrue(!TestPanel->CheckBox->IsChecked()));
			});
	}

	// ── Type tests ──────────────────────────────────────────────────

	TEST_METHOD(Type_EntersText)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));
		ASSERT_THAT(IsTrue(!TextBoxRef.IsEmpty()));

		const bool bTyped = USlateInspectorToolset::Type(TextBoxRef, TEXT("hello"));
		ASSERT_THAT(IsTrue(bTyped));
		ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("hello")));
	}

	TEST_METHOD(Type_AppendsToExistingText)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));

		USlateInspectorToolset::Type(TextBoxRef, TEXT("foo"));
		USlateInspectorToolset::Type(TextBoxRef, TEXT("bar"));
		ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("foobar")));
	}

	TEST_METHOD(Type_SpecialCharacters)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));

		USlateInspectorToolset::Type(TextBoxRef, TEXT("a@b#c$d"));
		ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("a@b#c$d")));
	}

	TEST_METHOD(Type_WithSubmit)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));
		ASSERT_THAT(IsTrue(!TextBoxRef.IsEmpty()));

		const bool bTyped = USlateInspectorToolset::Type(TextBoxRef, TEXT("submitted"), true);
		ASSERT_THAT(IsTrue(bTyped));
	}

	TEST_METHOD(Type_IntoSearchBox)
	{
		const FString Snapshot = SnapshotTestWindow();
		// SSearchBox contains an SEditableTextBox internally, so look for a textbox.
		// The search box is the third textbox in the panel (after TextBox and MultiLineTextBox).
		const TArray<FString> TextBoxRefs = FindAllRefsByRole(Snapshot, TEXT("textbox"));
		// We need at least 2 textbox refs (single-line and search box; multi-line may be a different role).
		ASSERT_THAT(IsTrue(TextBoxRefs.Num() >= 2));

		// Type into the last textbox (search box).
		const FString SearchRef = TextBoxRefs.Last();
		const bool bTyped = USlateInspectorToolset::Type(SearchRef, TEXT("search query"));
		ASSERT_THAT(IsTrue(bTyped));
	}

	TEST_METHOD(Type_InvalidRef)
	{
		const bool bTyped = USlateInspectorToolset::Type(TEXT("nonexistent_ref"), TEXT("hello"));
		ASSERT_THAT(IsTrue(!bTyped));
	}

	// ── PressKey in text context ────────────────────────────────────

	TEST_METHOD(PressKey_BackspaceDeletesCharacter)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));

				USlateInspectorToolset::Type(TextBoxRef, TEXT("abc"));
				ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("abc")));

				// BackSpace should delete the last character.
				// Re-focus via Type to ensure the editable text has focus.
				USlateInspectorToolset::Type(TextBoxRef, TEXT(""));
				USlateInspectorToolset::PressKey(TEXT("BackSpace"));
				ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("ab")));
			});
	}

	TEST_METHOD(PressKey_SelectAllAndDelete)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));

				USlateInspectorToolset::Type(TextBoxRef, TEXT("hello world"));
				ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("hello world")));

				// Ctrl+A to select all, then Delete to clear.
				USlateInspectorToolset::PressKey(TEXT("Ctrl+A"));
				USlateInspectorToolset::PressKey(TEXT("Delete"));
				ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString().IsEmpty()));
			});
	}

	// ── Hover tests ─────────────────────────────────────────────────

	TEST_METHOD(Hover_ReturnsTrue)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ButtonRef = FindRefByRole(Snapshot, TEXT("button"), TEXT("Test Button"));
				ASSERT_THAT(IsTrue(!ButtonRef.IsEmpty()));

				const bool bHovered = USlateInspectorToolset::Hover(ButtonRef);
				ASSERT_THAT(IsTrue(bHovered));
			});
	}

	TEST_METHOD(Hover_InvalidRef)
	{
		const bool bHovered = USlateInspectorToolset::Hover(TEXT("nonexistent_ref"));
		ASSERT_THAT(IsTrue(!bHovered));
	}

	// ── PressKey tests ──────────────────────────────────────────────

	TEST_METHOD(PressKey_Escape)
	{
		const bool bPressed = USlateInspectorToolset::PressKey(TEXT("Escape"));
		ASSERT_THAT(IsTrue(bPressed));
	}

	TEST_METHOD(PressKey_Tab)
	{
		const bool bPressed = USlateInspectorToolset::PressKey(TEXT("Tab"));
		ASSERT_THAT(IsTrue(bPressed));
	}

	// ── SelectOption tests ──────────────────────────────────────────

	TEST_METHOD(SelectOption_ChangesCombobox)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				ASSERT_THAT(IsTrue(*TestPanel->SelectedComboOption == TEXT("Alpha")));

				const FString Snapshot = SnapshotTestWindow();
				const FString ComboRef = FindRefByRole(Snapshot, TEXT("combobox"), TEXT(""));
				ASSERT_THAT(IsTrue(!ComboRef.IsEmpty()));

				const bool bSelected = USlateInspectorToolset::SelectOption(ComboRef, TEXT("Beta"));
				ASSERT_THAT(IsTrue(bSelected));
				ASSERT_THAT(IsTrue(*TestPanel->SelectedComboOption == TEXT("Beta")));
			});
	}

	TEST_METHOD(SelectOption_InvalidOption)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString ComboRef = FindRefByRole(Snapshot, TEXT("combobox"), TEXT(""));
				ASSERT_THAT(IsTrue(!ComboRef.IsEmpty()));

				const bool bSelected = USlateInspectorToolset::SelectOption(ComboRef, TEXT("Nonexistent"));
				ASSERT_THAT(IsTrue(!bSelected));
				// Original selection unchanged.
				ASSERT_THAT(IsTrue(*TestPanel->SelectedComboOption == TEXT("Alpha")));
			});
	}

	// ── FillForm tests ──────────────────────────────────────────────

	TEST_METHOD(FillForm_TextBox)
	{
		const FString Snapshot = SnapshotTestWindow();
		const FString TextBoxRef = FindRefByRole(Snapshot, TEXT("textbox"), TEXT(""));
		ASSERT_THAT(IsTrue(!TextBoxRef.IsEmpty()));

		TArray<FSlateInspectorToolsetFormField> Fields;
		FSlateInspectorToolsetFormField Field;
		Field.Ref = TextBoxRef;
		Field.Value = TEXT("form value");
		Field.FieldType = TEXT("textbox");
		Fields.Add(Field);

		const bool bFilled = USlateInspectorToolset::FillForm(Fields);
		ASSERT_THAT(IsTrue(bFilled));
		ASSERT_THAT(IsTrue(TestPanel->TextBox->GetText().ToString() == TEXT("form value")));
	}

	TEST_METHOD(FillForm_Checkbox)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString CheckBoxRef = FindRefByRole(Snapshot, TEXT("checkbox"), TEXT(""));
				ASSERT_THAT(IsTrue(!CheckBoxRef.IsEmpty()));
				ASSERT_THAT(IsTrue(!TestPanel->CheckBox->IsChecked()));

				TArray<FSlateInspectorToolsetFormField> Fields;
				FSlateInspectorToolsetFormField Field;
				Field.Ref = CheckBoxRef;
				Field.Value = TEXT("true");
				Field.FieldType = TEXT("checkbox");
				Fields.Add(Field);

				const bool bFilled = USlateInspectorToolset::FillForm(Fields);
				ASSERT_THAT(IsTrue(bFilled));
				ASSERT_THAT(IsTrue(TestPanel->CheckBox->IsChecked()));
			});
	}

	TEST_METHOD(FillForm_CheckboxAlreadyInDesiredState)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]()
			{
				const FString Snapshot = SnapshotTestWindow();
				const FString CheckBoxRef = FindRefByRole(Snapshot, TEXT("checkbox"), TEXT(""));
				ASSERT_THAT(IsTrue(!TestPanel->CheckBox->IsChecked()));

				TArray<FSlateInspectorToolsetFormField> Fields;
				FSlateInspectorToolsetFormField Field;
				Field.Ref = CheckBoxRef;
				Field.Value = TEXT("false");
				Field.FieldType = TEXT("checkbox");
				Fields.Add(Field);

				// Should succeed without clicking (already unchecked).
				const bool bFilled = USlateInspectorToolset::FillForm(Fields);
				ASSERT_THAT(IsTrue(bFilled));
				ASSERT_THAT(IsTrue(!TestPanel->CheckBox->IsChecked()));
			});
	}

	// ── WaitFor tests ───────────────────────────────────────────────

	TEST_METHOD(WaitFor_FindsExistingText)
	{
		const bool bFound = USlateInspectorToolset::WaitFor(TEXT("Static Label"));
		ASSERT_THAT(IsTrue(bFound));
	}

	TEST_METHOD(WaitFor_DoesNotFindMissingText)
	{
		const bool bFound = USlateInspectorToolset::WaitFor(TEXT("This Text Does Not Exist"));
		ASSERT_THAT(IsTrue(!bFound));
	}

	TEST_METHOD(WaitFor_TextGone)
	{
		const bool bGone = USlateInspectorToolset::WaitFor(TEXT(""), TEXT("This Text Does Not Exist"));
		ASSERT_THAT(IsTrue(bGone));
	}

	TEST_METHOD(WaitFor_TextGoneFails)
	{
		const bool bGone = USlateInspectorToolset::WaitFor(TEXT(""), TEXT("Static Label"));
		ASSERT_THAT(IsTrue(!bGone));
	}

	// ── Windows tests ───────────────────────────────────────────────

	TEST_METHOD(Windows_ListContainsTestWindow)
	{
		const FString WindowList = USlateInspectorToolset::Windows(TEXT("list"));
		ASSERT_THAT(IsTrue(WindowList.Contains(TEXT("SlateInspectorToolset Test Window"))));
	}

	// ── Ref cache tests ─────────────────────────────────────────────

	TEST_METHOD(PurgeExpired_CleansDeadRefs)
	{
		TSharedRef<STextBlock> EphemeralWidget = SNew(STextBlock).Text(FText::FromString(TEXT("Ephemeral")));
		const FString Ref = FSlateInspectorToolsetRefCache::Get().GetOrAssignRef(EphemeralWidget, TEXT("x"));
		ASSERT_THAT(IsTrue(!Ref.IsEmpty()));

		TSharedPtr<SWidget> Resolved = FSlateInspectorToolsetRefCache::Get().ResolveRef(Ref);
		ASSERT_THAT(IsTrue(Resolved.IsValid()));

		// Drop all references so the widget is destroyed.
		Resolved.Reset();
		EphemeralWidget = SNew(STextBlock);

		FSlateInspectorToolsetRefCache::Get().PurgeExpired();
		TSharedPtr<SWidget> ResolvedAfterPurge = FSlateInspectorToolsetRefCache::Get().ResolveRef(Ref);
		ASSERT_THAT(IsTrue(!ResolvedAfterPurge.IsValid()));
	}

	TEST_METHOD(FindRef_ReturnsEmptyForUnknownWidget)
	{
		TSharedRef<STextBlock> Widget = SNew(STextBlock);
		const FString Ref = FSlateInspectorToolsetRefCache::Get().FindRef(Widget);
		ASSERT_THAT(IsTrue(Ref.IsEmpty()));
	}

	TEST_METHOD(ResolveRef_ReturnsNullForUnknownRef)
	{
		TSharedPtr<SWidget> Resolved = FSlateInspectorToolsetRefCache::Get().ResolveRef(TEXT("zzz999"));
		ASSERT_THAT(IsTrue(!Resolved.IsValid()));
	}

	// ── Helpers ─────────────────────────────────────────────────────

	/** Snapshot only the test window's subtree so we don't accidentally
	 *  match editor widgets (e.g. editor checkboxes vs. our test checkbox). */
	FString SnapshotTestWindow()
	{
		// First get the window ref from a full snapshot.
		const FString FullSnapshot = USlateInspectorToolset::Snapshot(TEXT(""), 30);
		const FString WindowRef = FindRefByRole(FullSnapshot, TEXT("window"), TEXT("SlateInspectorToolset Test Window"));
		if (WindowRef.IsEmpty())
		{
			return FString();
		}
		return USlateInspectorToolset::Snapshot(WindowRef, 30);
	}

	FString FindRefByRole(const FString& Snapshot, const FString& Role, const FString& Label)
	{
		TArray<FString> Lines;
		Snapshot.ParseIntoArrayLines(Lines);
		for (const FString& Line : Lines)
		{
			FString Trimmed = Line.TrimStartAndEnd();
			if (!Trimmed.StartsWith(Role))
			{
				continue;
			}
			if (!Label.IsEmpty() && !Trimmed.Contains(Label))
			{
				continue;
			}
			int32 RefStart = Trimmed.Find(TEXT("[ref="));
			if (RefStart != INDEX_NONE)
			{
				RefStart += 5;
				int32 RefEnd = Trimmed.Find(TEXT("]"), ESearchCase::IgnoreCase, ESearchDir::FromStart, RefStart);
				if (RefEnd != INDEX_NONE)
				{
					return Trimmed.Mid(RefStart, RefEnd - RefStart);
				}
			}
		}
		return FString();
	}

	TArray<FString> FindAllRefsByRole(const FString& Snapshot, const FString& Role)
	{
		TArray<FString> Refs;
		TArray<FString> Lines;
		Snapshot.ParseIntoArrayLines(Lines);
		for (const FString& Line : Lines)
		{
			FString Trimmed = Line.TrimStartAndEnd();
			if (!Trimmed.StartsWith(Role))
			{
				continue;
			}
			int32 RefStart = Trimmed.Find(TEXT("[ref="));
			if (RefStart != INDEX_NONE)
			{
				RefStart += 5;
				int32 RefEnd = Trimmed.Find(TEXT("]"), ESearchCase::IgnoreCase, ESearchDir::FromStart, RefStart);
				if (RefEnd != INDEX_NONE)
				{
					Refs.Add(Trimmed.Mid(RefStart, RefEnd - RefStart));
				}
			}
		}
		return Refs;
	}
};
