// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FSlateInspectorToolsetTestsModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FSlateInspectorToolsetTestsModule, SlateInspectorToolsetTests)
